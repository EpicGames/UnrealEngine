// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildStep.h"

namespace UGSCore
{

const TCHAR* FBuildStep::UniqueIdKey = TEXT("UniqueId");

bool TryParse(const TCHAR* Text, EBuildStepType& OutType)
{
	if (FCString::Stricmp(Text, TEXT("Compile")) == 0)
	{
		OutType = EBuildStepType::Compile;
		return true;
	}
	if (FCString::Stricmp(Text, TEXT("Cook")) == 0)
	{
		OutType = EBuildStepType::Cook;
		return true;
	}
	if (FCString::Stricmp(Text, TEXT("Other")) == 0)
	{
		OutType = EBuildStepType::Other;
		return true;
	}
	return false;
}

const TCHAR* ToString(EBuildStepType Type)
{
	switch(Type)
	{
	case EBuildStepType::Compile:
		return TEXT("Compile");
	case EBuildStepType::Cook:
		return TEXT("Cook");
	case EBuildStepType::Other:
		return TEXT("Other");
	default:
		return TEXT("<Invalid>");
	}
}

FBuildStep::FBuildStep(const FGuid& InUniqueId, int InOrderIndex, const FString& InDescription, const FString& InStatusText, int InEstimatedDuration, const FString& InTarget, const FString& InPlatform, const FString& InConfiguration, const FString& InArguments, bool bInSyncDefault)
	: UniqueId(InUniqueId)
	, OrderIndex(InOrderIndex)
	, Description(InDescription)
	, StatusText(InStatusText)
	, EstimatedDuration(InEstimatedDuration)
	, Type(EBuildStepType::Compile)
	, Target(InTarget)
	, Platform(InPlatform)
	, Configuration(InConfiguration)
	, Arguments(InArguments)
	, bUseLogWindow(false)
	, bNormalSync(bInSyncDefault)
	, bScheduledSync(bInSyncDefault)
	, bShowAsTool(false)
{
}

FBuildStep::FBuildStep(const FCustomConfigObject& Object)
{
	if(!Object.TryGetValue(UniqueIdKey, UniqueId))
	{
		UniqueId = FGuid::NewGuid();
	}
	if(!Object.TryGetValue(TEXT("OrderIndex"), OrderIndex))
	{
		OrderIndex = -1;
	}

	Description = Object.GetValueOrDefault(TEXT("Description"), TEXT("Untitled"));
	StatusText = Object.GetValueOrDefault(TEXT("StatusText"), TEXT("Untitled"));

	if(!Object.TryGetValue(TEXT("EstimatedDuration"), EstimatedDuration) || EstimatedDuration < 1)
	{
		EstimatedDuration = 1;
	}

	if(!TryParse(Object.GetValueOrDefault(TEXT("Type"), TEXT("")), Type))
	{
		Type = EBuildStepType::Other;
	}

	Object.TryGetValue(TEXT("Target"), Target);
	Object.TryGetValue(TEXT("Platform"), Platform);
	Object.TryGetValue(TEXT("Configuration"), Configuration);
	Object.TryGetValue(TEXT("FileName"), FileName);
	Object.TryGetValue(TEXT("Arguments"), Arguments);

	bUseLogWindow = Object.GetValueOrDefault(TEXT("bUseLogWindow"), true);
	bNormalSync = Object.GetValueOrDefault(TEXT("bNormalSync"), true);
	bScheduledSync = Object.GetValueOrDefault(TEXT("bScheduledSync"), true);
	bShowAsTool = Object.GetValueOrDefault(TEXT("bShowAsTool"), false);
}

void FBuildStep::MergeBuildStepObjects(TMap<FGuid, FCustomConfigObject>& BuildStepObjects, const TArray<FCustomConfigObject>& ModifyObjects)
{
	for(const FCustomConfigObject& ModifyObject : ModifyObjects)
	{
		FGuid UniqueId;
		if(ModifyObject.TryGetValue(UniqueIdKey, UniqueId))
		{
			FCustomConfigObject CombinedObject = ModifyObject;

			FCustomConfigObject* DefaultObject = BuildStepObjects.Find(UniqueId);
			if(DefaultObject != nullptr)
			{
				CombinedObject.SetDefaults(*DefaultObject);
			}

			BuildStepObjects[UniqueId] = CombinedObject;
		}
	}
}

FCustomConfigObject FBuildStep::ToConfigObject() const
{
	FCustomConfigObject Result;
	Result.SetValue(TEXT("UniqueId"), UniqueId);
	Result.SetValue(TEXT("Description"), *Description);
	Result.SetValue(TEXT("StatusText"), *StatusText);
	Result.SetValue(TEXT("EstimatedDuration"), EstimatedDuration);
	Result.SetValue(TEXT("Type"), ToString(Type));
	switch(Type)
	{
	case EBuildStepType::Compile:
		Result.SetValue(TEXT("Target"), *Target);
		Result.SetValue(TEXT("Platform"), *Platform);
		Result.SetValue(TEXT("Configuration"), *Configuration);
		Result.SetValue(TEXT("Arguments"), *Arguments);
		break;
	case EBuildStepType::Cook:
		Result.SetValue(TEXT("FileName"), *FileName);
		break;
	case EBuildStepType::Other:
		Result.SetValue(TEXT("FileName"), *FileName);
		Result.SetValue(TEXT("Arguments"), *Arguments);
		Result.SetValue(TEXT("bUseLogWindow"), bUseLogWindow);
		break;
	}
	Result.SetValue(TEXT("OrderIndex"), OrderIndex);
	Result.SetValue(TEXT("bNormalSync"), bNormalSync);
	Result.SetValue(TEXT("bScheduledSync"), bScheduledSync);
	Result.SetValue(TEXT("bShowAsTool"), bShowAsTool);
	return Result;
}

FCustomConfigObject FBuildStep::ToConfigObject(const FCustomConfigObject* DefaultObject) const
{
	FCustomConfigObject Result;
	Result.SetValue(UniqueIdKey, UniqueId);
	Result.AddOverrides(ToConfigObject(), DefaultObject);
	return Result;
}

} // namespace UGSCore
