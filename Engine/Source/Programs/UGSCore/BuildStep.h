// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "CustomConfigFile.h"

namespace UGSCore
{

enum class EBuildStepType
{
	Compile,
	Cook,
	Other,
};

bool TryParse(const TCHAR* Text, EBuildStepType& OutType);
const TCHAR* ToString(EBuildStepType Type);

struct FBuildStep
{
	static const TCHAR* UniqueIdKey;

	FGuid UniqueId;
	int OrderIndex;
	FString Description;
	FString StatusText;
	int EstimatedDuration;
	EBuildStepType Type;
	FString Target;
	FString Platform;
	FString Configuration;
	FString FileName;
	FString Arguments;
	bool bUseLogWindow;
	bool bNormalSync;
	bool bScheduledSync;
	bool bShowAsTool;

	FBuildStep(const FGuid& InUniqueId, int InOrderIndex, const FString& InDescription, const FString& InStatusText, int InEstimatedDuration, const FString& InTarget, const FString& InPlatform, const FString& InConfiguration, const FString& InArguments, bool bInSyncDefault);
	FBuildStep(const FCustomConfigObject& Object);

	static void MergeBuildStepObjects(TMap<FGuid, FCustomConfigObject>& BuildStepObjects, const TArray<FCustomConfigObject>& ModifyObjects);

	FCustomConfigObject ToConfigObject() const;
	FCustomConfigObject ToConfigObject(const FCustomConfigObject* DefaultObject) const;
};

} // namespace UGSCore
