// Copyright Epic Games, Inc. All Rights Reserved.

#include "CreateClientTask.h"

#define LOCTEXT_NAMESPACE "UnrealGameSync"

namespace UGSCore
{

	FCreateClientTask::FCreateClientTask(
		TSharedRef<FPerforceConnection> InPerforce,
		TSharedRef<FLineBasedTextWriter> InLog,
		const FPerforceClientRecord& InClientRecord,
		const FString& InStream)
		: Perforce(MoveTemp(InPerforce))
		, Log(MoveTemp(InLog))
		, ClientRecord(InClientRecord)
		, Stream(InStream)
	{}

	FCreateClientTask::~FCreateClientTask()
	{}

	TSharedRef<FModalTaskResult> FCreateClientTask::Run(FEvent* AbortEvent)
	{
		return RunInternal(AbortEvent);
	}

	TSharedRef<FModalTaskResult> FCreateClientTask::RunInternal(FEvent* AbortEvent) const
	{
		return Perforce->CreateClient(ClientRecord, Stream, AbortEvent, *Log)
			? FModalTaskResult::Success()
			: FModalTaskResult::Failure(LOCTEXT("CreateClientTaskFailure", "Failed to find streams"));
	}

#undef LOCTEXT_NAMESPACE

} // namespace UGSCore
