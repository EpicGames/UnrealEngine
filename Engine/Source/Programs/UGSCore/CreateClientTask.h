// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ModalTask.h"
#include "Perforce.h"
#include "OutputAdapters.h"

namespace UGSCore
{

class FCreateClientTask : public IModalTask
{
public:
	TSharedRef<FPerforceConnection> Perforce;

	FCreateClientTask(
		TSharedRef<FPerforceConnection> InPerforce,
		TSharedRef<FLineBasedTextWriter> InLog,
		const FPerforceClientRecord& InClientRecord,
		const FString& InStream);
	virtual ~FCreateClientTask() override;

	virtual TSharedRef<FModalTaskResult> Run(FEvent* AbortEvent) override;

private:
	TSharedRef<FLineBasedTextWriter> Log;

	const FPerforceClientRecord& ClientRecord;
	const FString& Stream;

	TSharedRef<FModalTaskResult> RunInternal(FEvent* AbortEvent) const;
};

} // namespace UGSCore
