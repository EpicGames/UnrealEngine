// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomConfigFile.h"
#include "Misc/FileHelper.h"
#include "Utility.h"

namespace UGSCore
{

static const TCHAR* ConfigSeparatorCharacters = TEXT("(),= \t\"");

//// FCustomConfigObject ////

FCustomConfigObject::FCustomConfigObject()
{
}

FCustomConfigObject::FCustomConfigObject(const TCHAR* Text)
{
	ParseConfigString(Text);
}

FCustomConfigObject::FCustomConfigObject(const FCustomConfigObject& BaseObject, const TCHAR* Text)
	: Pairs(BaseObject.Pairs)
{
	ParseConfigString(Text);
}

void FCustomConfigObject::ParseConfigString(const TCHAR* Text)
{
	int Idx = 0;
	if(ParseConfigToken(Text, Idx) == "(")
	{
		while(Text[Idx] != 0)
		{
			// Read the next key/value pair
			FString Key = ParseConfigToken(Text, Idx);
			if(ParseConfigToken(Text, Idx) == "=")
			{
				FString Value = ParseConfigToken(Text, Idx);
				SetValue(*Key, *Value);
			}

			// Check for the end of the list, or a comma before the next pair
			for(;;)
			{
				FString Token = ParseConfigToken(Text, Idx);
				if(Token == ",")
				{
					break;
				}
				if(Token == ")" || Token.Len() == 0)
				{
					return;
				}
			}
		}
	}
}

FString FCustomConfigObject::ParseConfigToken(const TCHAR* Text, int& Idx)
{
	// Skip whitespace
	while(Text[Idx] != 0 && FChar::IsWhitespace(Text[Idx]))
	{
		Idx++;
	}
	if(Text[Idx] == 0)
	{
		return FString();
	}

	// Read the token
	if(Text[Idx] == '\"')
	{
		FString Token;
		while(Text[++Idx] != 0)
		{
			if(Text[Idx] == '\"')
			{
				Idx++;
				break;
			}
			if(Text[Idx] == '\\' && Text[Idx + 1] != 0)
			{
				Idx++;
			}
			Token += Text[Idx];
		}
		return Token;
	}
	else if(FCString::Strchr(ConfigSeparatorCharacters, Text[Idx]) != nullptr)
	{
		FString Token;
		Token.AppendChar(Text[Idx++]);
		return Token;
	}
	else
	{
		int StartIdx = Idx;
		while(Text[Idx] != 0 && FCString::Strchr(ConfigSeparatorCharacters, Text[Idx]) == nullptr)
		{
			Idx++;
		}
		return FString(Idx - StartIdx, Text + StartIdx);
	}
}

bool FCustomConfigObject::HasKey(const TCHAR* Key) const
{
	const TCHAR* CurrentValue;
	return TryGetValue(Key, CurrentValue);
}

bool FCustomConfigObject::HasValue(const TCHAR* Key, const TCHAR* Value) const
{
	const TCHAR* CurrentValue;
	return TryGetValue(Key, CurrentValue) && FCString::Stricmp(CurrentValue, Value) == 0;
}

bool FCustomConfigObject::TryGetValue(const TCHAR* Key, const TCHAR*& OutValue) const
{
	for(int Idx = 0; Idx < Pairs.Num(); Idx++)
	{
		if(Pairs[Idx].Key == Key)
		{
			OutValue = *Pairs[Idx].Value;
			return true;
		}
	}
	return false;
}

bool FCustomConfigObject::TryGetValue(const TCHAR* Key, FString& OutValue) const
{
	const TCHAR* Value;
	if(TryGetValue(Key, Value))
	{
		OutValue = Value;
		return true;
	}
	return false;
}

bool FCustomConfigObject::TryGetValue(const TCHAR* Key, FGuid& OutValue) const
{
	const TCHAR* StringValue;
	if(TryGetValue(Key, StringValue))
	{
		FGuid GuidValue;
		if(FGuid::Parse(StringValue, GuidValue))
		{
			OutValue = GuidValue;
			return true;
		}
	}
	return false;
}

bool FCustomConfigObject::TryGetValue(const TCHAR* Key, int& OutValue) const
{
	const TCHAR* StringValue;
	if(TryGetValue(Key, StringValue))
	{
		TCHAR* EndPtr = nullptr;
		int IntegerValue = FCString::Strtoi(StringValue, &EndPtr, 10);
		if(*EndPtr == 0)
		{
			OutValue = IntegerValue;
			return true;
		}
	}
	return false;
}

bool FCustomConfigObject::TryGetValue(const TCHAR* Key, bool& OutValue) const
{
	const TCHAR* StringValue;
	if(TryGetValue(Key, StringValue))
	{
		if(FCString::Stricmp(StringValue, TEXT("True")) == 0)
		{
			OutValue = true;
			return true;
		}
		else if(FCString::Stricmp(StringValue, TEXT("False")) == 0)
		{
			OutValue = false;
			return true;
		}
	}
	return false;
}

const TCHAR* FCustomConfigObject::GetValueOrDefault(const TCHAR* Key, const TCHAR* DefaultValue) const
{
	const TCHAR* Value;
	if(TryGetValue(Key, Value))
	{
		return Value;
	}
	else
	{
		return DefaultValue;
	}
}

FGuid FCustomConfigObject::GetValueOrDefault(const TCHAR* Key, const FGuid& DefaultValue) const
{
	FGuid Value;
	if(TryGetValue(Key, Value))
	{
		return Value;
	}
	else
	{
		return DefaultValue;
	}
}

int FCustomConfigObject::GetValueOrDefault(const TCHAR* Key, int DefaultValue) const
{
	int Value;
	if(TryGetValue(Key, Value))
	{
		return Value;
	}
	else
	{
		return DefaultValue;
	}
}

bool FCustomConfigObject::GetValueOrDefault(const TCHAR* Key, bool DefaultValue) const
{
	bool Value;
	if(TryGetValue(Key, Value))
	{
		return Value;
	}
	else
	{
		return DefaultValue;
	}
}

void FCustomConfigObject::SetValue(const TCHAR* Key, const TCHAR* Value)
{
	for(int Idx = 0; Idx < Pairs.Num(); Idx++)
	{
		if(Pairs[Idx].Key == Key)
		{
			Pairs[Idx] = TTuple<FString,FString>(Key, Value);
			return;
		}
	}
	Pairs.Add(TTuple<FString,FString>(Key, Value));
}

void FCustomConfigObject::SetValue(const TCHAR* Key, const FGuid& Value)
{
	SetValue(Key, *Value.ToString());
}

void FCustomConfigObject::SetValue(const TCHAR* Key, int Value)
{
	SetValue(Key, *FString::Printf(TEXT("%d"), Value));
}

void FCustomConfigObject::SetValue(const TCHAR* Key, bool Value)
{
	SetValue(Key, Value? TEXT("True") : TEXT("False"));
}

void FCustomConfigObject::SetDefaults(const FCustomConfigObject& Other)
{
	for(const TTuple<FString, FString>& Pair : Other.Pairs)
	{
		if(!HasKey(*Pair.Key))
		{
			SetValue(*Pair.Key, *Pair.Value);
		}
	}
}

void FCustomConfigObject::AddOverrides(const FCustomConfigObject& Object, const FCustomConfigObject* DefaultObject)
{
	for(const TTuple<FString, FString>& Pair : Object.Pairs)
	{
		if(DefaultObject == nullptr || !DefaultObject->HasValue(*Pair.Key, *Pair.Value))
		{
			SetValue(*Pair.Key, *Pair.Value);
		}
	}
}

FString FCustomConfigObject::ToString(const FCustomConfigObject* BaseObject) const
{
	FString Result;
	Result.Append("(");
	for(const TTuple<FString, FString>& Pair : Pairs)
	{
		if(BaseObject == nullptr || !BaseObject->HasValue(*Pair.Key, *Pair.Value))
		{
			if(Result.Len() > 1)
			{
				Result.Append(", ");
			}
			Result.Append(Pair.Key);
			Result.Append("=");
			if(Pair.Value.Len() == 0)
			{
				Result.Append("\"\"");
			}
			else
			{
				Result.Append(FString(TEXT("\"")) + Pair.Value.Replace(TEXT("\\"), TEXT("\\\\")).Replace(TEXT("\""), TEXT("\\\"")) + FString(TEXT("\"")));
			}
		}
	}
	Result.Append(")");
	return Result;
}
 
FString FCustomConfigObject::ToString() const
{
	return ToString(nullptr);
}

//// FCustomConfigSection ////

FCustomConfigSection::FCustomConfigSection(FString&& InName)
	: Name(InName)
{
}

void FCustomConfigSection::Clear()
{
	Pairs.Empty();
}

void FCustomConfigSection::SetValue(const TCHAR* Key, int Value)
{
	Pairs.FindOrAdd(Key) = FString::Printf(TEXT("%d"), Value);
}

void FCustomConfigSection::SetValue(const TCHAR* Key, bool Value)
{
	Pairs.FindOrAdd(Key) = Value? "1" : "0";
}

void FCustomConfigSection::SetValue(const TCHAR* Key, const TCHAR* Value)
{
	if(Value == nullptr)
	{
		RemoveValue(Key);
	}
	else
	{
		Pairs.FindOrAdd(Key) = Value;
	}
}

void FCustomConfigSection::SetValues(const TCHAR* Key, const TArray<FString>& Values)
{
	if(Values.Num() == 0)
	{
		RemoveValue(Key);
	}
	else
	{
		Pairs.FindOrAdd(Key) = FString::Join(Values, TEXT("\n"));
	}
}

void FCustomConfigSection::SetValues(const TCHAR* Key, const TArray<FGuid>& Values)
{
	if(Values.Num() == 0)
	{
		RemoveValue(Key);
	}
	else
	{
		FString Value = Values[0].ToString();
		for(int Idx = 1; Idx < Values.Num(); Idx++)
		{
			Value += "\n";
			Value += Values[Idx].ToString();
		}
		Pairs.FindOrAdd(Key) = Value;
	}
}

void FCustomConfigSection::SetValues(const TCHAR* Key, const TArray<int>& Values)
{
	if(Values.Num() == 0)
	{
		RemoveValue(Key);
	}
	else
	{
		FString Value = FString::Printf(TEXT("%d"), Values[0]);
		for(int Idx = 1; Idx < Values.Num(); Idx++)
		{
			Value += TEXT("\n");
			Value += FString::Printf(TEXT("%d"), Values[Idx]);
		}
		Pairs.FindOrAdd(Key) = Value;
	}
}

void FCustomConfigSection::AppendValue(const TCHAR* Key, const TCHAR* Value)
{
	FString* CurrentValue = Pairs.Find(Key);
	if(CurrentValue != nullptr)
	{
		(*CurrentValue) += FString::Printf(TEXT("\n%s"), Value);
	}
	else
	{
		Pairs.FindOrAdd(Key) = Value;
	}
}

void FCustomConfigSection::RemoveValue(const TCHAR* Key)
{
	Pairs.Remove(Key);
}

int FCustomConfigSection::GetValue(const TCHAR* Key, int DefaultValue) const
{
	const TCHAR* ValueString = GetValue(Key);
	if(ValueString != nullptr)
	{
		TCHAR* EndPtr = nullptr;
		int Value = FCString::Strtoi(ValueString, &EndPtr, 10);
		if(*EndPtr == 0)
		{
			return Value;
		}
	}
	return DefaultValue;
}

bool FCustomConfigSection::GetValue(const TCHAR* Key, bool DefaultValue) const
{
	return GetValue(Key, DefaultValue? 1 : 0) != 0;
}

const TCHAR* FCustomConfigSection::GetValue(const TCHAR* Key, const TCHAR* DefaultValue) const
{
	const FString* Value = Pairs.Find(Key);
	if(Value == nullptr)
	{
		return DefaultValue;
	}
	else
	{
		return **Value;
	}
}

bool FCustomConfigSection::TryGetValue(const TCHAR* Key, FString& OutValue) const
{
	const TCHAR* ValueString = GetValue(Key);
	if(ValueString != nullptr)
	{
		OutValue = ValueString;
		return true;
	}
	return false;
}

bool FCustomConfigSection::TryGetValue(const TCHAR* Key, int32& OutValue) const
{
	const TCHAR* ValueString = GetValue(Key);
	if(ValueString != nullptr)
	{
		TCHAR* EndPtr = nullptr;
		int Value = FCString::Strtoi(ValueString, &EndPtr, 10);
		if(*EndPtr == 0)
		{
			OutValue = Value;
			return true;
		}
	}
	return false;
}

bool FCustomConfigSection::TryGetValues(const TCHAR* Key, TArray<FString>& Values) const
{
	const FString* Value = Pairs.Find(Key);
	if(Value != nullptr)
	{
		Value->ParseIntoArray(Values, TEXT("\n"), false);
		return true;
	}
	return false;
}

bool FCustomConfigSection::TryGetValues(const TCHAR* Key, TArray<FGuid>& Values) const
{
	TArray<FString> StringValues;
	if(TryGetValues(Key, StringValues))
	{
		for(const FString& StringValue : StringValues)
		{
			FGuid Guid;
			if(FGuid::Parse(StringValue, Guid))
			{
				Values.Add(Guid);
			}
		}
		return true;
	}
	return false;
}

bool FCustomConfigSection::TryGetValues(const TCHAR* Key, TArray<int>& Values) const
{
	TArray<FString> StringValues;
	if(TryGetValues(Key, StringValues))
	{
		for(const FString& StringValue : StringValues)
		{
			int IntegerValue;
			if(FUtility::TryParse(*StringValue, IntegerValue))
			{
				Values.Add(IntegerValue);
			}
		}
		return true;
	}
	return false;
}

//// FCustomConfigFile ////

FCustomConfigFile::FCustomConfigFile()
{
}

void FCustomConfigFile::Load(const TCHAR* FileName)
{
	TArray<FString> Lines;
	FFileHelper::LoadFileToStringArray(Lines, FileName);
	Parse(Lines);
}

void FCustomConfigFile::Parse(const TArray<FString>& Lines)
{
	TSharedPtr<FCustomConfigSection> CurrentSection;
	for(const FString& Line : Lines)
	{
		FString TrimLine = Line.TrimStartAndEnd();
		if(!TrimLine.StartsWith(";"))
		{
			if(TrimLine.StartsWith("[") && TrimLine.EndsWith("]"))
			{
				FString SectionName = TrimLine.Mid(1, TrimLine.Len() - 2).TrimStartAndEnd();
				CurrentSection = FindOrAddSection(*SectionName);
			}
			else if(CurrentSection.IsValid())
			{
				int EqualsIdx;
				if(TrimLine.FindChar(TEXT('='), EqualsIdx))
				{
					FString Value = Line.Mid(EqualsIdx + 1).TrimStart();
					if(TrimLine.StartsWith("+"))
					{
						FString Key = TrimLine.Mid(1, EqualsIdx - 1).TrimStartAndEnd();
						CurrentSection->AppendValue(*Key, *Value);
					}
					else
					{
						FString Key = TrimLine.Mid(0, EqualsIdx).TrimEnd();
						CurrentSection->SetValue(*Key, *Value);
					}
				}
			}
		}
	}
}

void FCustomConfigFile::Save(const TCHAR* FileName)
{
	TArray<FString> Lines;
	for(int Idx = 0; Idx < Sections.Num(); Idx++)
	{
		Lines.Add(FString::Printf(TEXT("[%s]"), *Sections[Idx]->Name));
		for(const TTuple<FString, FString>& Pair : Sections[Idx]->Pairs)
		{
			if(Pair.Value.Contains(TEXT("\n")))
			{
				TArray<FString> Values;
				Pair.Value.ParseIntoArrayLines(Values, false);

				for(const FString& Value : Values)
				{
					Lines.Add(FString::Printf(TEXT("+%s=%s"), *Pair.Key, *Value));
				}
			}
			else
			{
				Lines.Add(FString::Printf(TEXT("%s=%s"), *Pair.Key, *Pair.Value));
			}
		}
		if(Idx < Sections.Num() - 1)
		{
			Lines.Add(FString());
		}
	}

	FFileHelper::SaveStringArrayToFile(Lines, FileName);
}

TSharedPtr<FCustomConfigSection> FCustomConfigFile::FindSection(const TCHAR* Name)
{
	for(const TSharedPtr<FCustomConfigSection>& Section : Sections)
	{
		if(Section->Name == Name)
		{
			return Section;
		}
	}
	return nullptr;
}

TSharedPtr<const FCustomConfigSection> FCustomConfigFile::FindSection(const TCHAR* Name) const
{
	for(const TSharedPtr<FCustomConfigSection>& Section : Sections)
	{
		if(Section->Name == Name)
		{
			return Section;
		}
	}
	return nullptr;
}

TSharedRef<FCustomConfigSection> FCustomConfigFile::FindOrAddSection(const TCHAR* Name)
{
	TSharedPtr<FCustomConfigSection> Section = FindSection(Name);
	if(!Section.IsValid())
	{
		TSharedRef<FCustomConfigSection> NewSection = MakeShared<FCustomConfigSection>(Name);
		Sections.Add(NewSection);
		return NewSection;
	}
	return Section.ToSharedRef();
}

void FCustomConfigFile::SetValue(const TCHAR* Key, int Value)
{
	const TCHAR* Dot = FCString::Strchr(Key, TEXT('.'));
	check(Dot != nullptr);
	TSharedRef<FCustomConfigSection> Section = FindOrAddSection(*FString(Dot - Key, Key));
	Section->SetValue(Dot + 1, Value);
}

void FCustomConfigFile::SetValue(const TCHAR* Key, bool Value)
{
	const TCHAR* Dot = FCString::Strchr(Key, TEXT('.'));
	check(Dot != nullptr);
	TSharedRef<FCustomConfigSection> Section = FindOrAddSection(*FString(Dot - Key, Key));
	Section->SetValue(Dot + 1, Value);
}

void FCustomConfigFile::SetValue(const TCHAR* Key, const TCHAR* Value)
{
	const TCHAR* Dot = FCString::Strchr(Key, TEXT('.'));
	check(Dot != nullptr);
	TSharedRef<FCustomConfigSection> Section = FindOrAddSection(*FString(Dot - Key, Key));
	Section->SetValue(Dot + 1, Value);
}

void FCustomConfigFile::SetValues(const TCHAR* Key, const TArray<FString>& Values)
{
	const TCHAR* Dot = FCString::Strchr(Key, TEXT('.'));
	check(Dot != nullptr);
	TSharedRef<FCustomConfigSection> Section = FindOrAddSection(*FString(Dot - Key, Key));
	Section->SetValues(Dot + 1, Values);
}

bool FCustomConfigFile::GetValue(const TCHAR* Key, bool DefaultValue) const
{
	const TCHAR* Dot = FCString::Strchr(Key, TEXT('.'));
	check(Dot != nullptr);
	TSharedPtr<const FCustomConfigSection> Section = FindSection(*FString(Dot - Key, Key));
	return Section.IsValid()? Section->GetValue(Dot + 1, DefaultValue) : DefaultValue;
}

int32 FCustomConfigFile::GetValue(const TCHAR* Key, int32 DefaultValue) const
{
	const TCHAR* Dot = FCString::Strchr(Key, TEXT('.'));
	check(Dot != nullptr);
	TSharedPtr<const FCustomConfigSection> Section = FindSection(*FString(Dot - Key, Key));
	return Section.IsValid()? Section->GetValue(Dot + 1, DefaultValue) : DefaultValue;
}

const TCHAR* FCustomConfigFile::GetValue(const TCHAR* Key, const TCHAR* DefaultValue) const
{
	const TCHAR* Dot = FCString::Strchr(Key, TEXT('.'));
	check(Dot != nullptr);
	TSharedPtr<const FCustomConfigSection> Section = FindSection(*FString(Dot - Key, Key));
	return Section.IsValid()? Section->GetValue(Dot + 1, DefaultValue) : DefaultValue;
}

bool FCustomConfigFile::TryGetValue(const TCHAR* Key, FString& OutValue) const
{
	const TCHAR* Dot = FCString::Strchr(Key, TEXT('.'));
	check(Dot != nullptr);
	TSharedPtr<const FCustomConfigSection> Section = FindSection(*FString(Dot - Key, Key));
	return Section.IsValid()? Section->TryGetValue(Key, OutValue) : false;
}

bool FCustomConfigFile::TryGetValue(const TCHAR* Key, int32& OutValue) const
{
	const TCHAR* Dot = FCString::Strchr(Key, TEXT('.'));
	check(Dot != nullptr);
	TSharedPtr<const FCustomConfigSection> Section = FindSection(*FString(Dot - Key, Key));
	return Section.IsValid()? Section->TryGetValue(Key, OutValue) : false;
}

bool FCustomConfigFile::TryGetValues(const TCHAR* Key, TArray<FString>& Values) const
{
	const TCHAR* Dot = FCString::Strchr(Key, TEXT('.'));
	check(Dot != nullptr);
	TSharedPtr<const FCustomConfigSection> Section = FindSection(*FString(Dot - Key, Key));
	return Section.IsValid()? Section->TryGetValues(Dot + 1, Values) : false;
}

bool FCustomConfigFile::TryGetValues(const TCHAR* Key, TArray<FGuid> &Values) const
{
	const TCHAR* Dot = FCString::Strchr(Key, TEXT('.'));
	check(Dot != nullptr);
	TSharedPtr<const FCustomConfigSection> Section = FindSection(*FString(Dot - Key, Key));
	return Section.IsValid()? Section->TryGetValues(Dot + 1, Values) : false;
}

} // namespace UGSCore
