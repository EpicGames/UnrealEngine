// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/Tuple.h"

namespace UGSCore
{

struct FCustomConfigObject
{
	TArray<TTuple<FString, FString>> Pairs;

	FCustomConfigObject();
	FCustomConfigObject(const TCHAR* Text);
	FCustomConfigObject(const FCustomConfigObject& BaseObject, const TCHAR* Text);

	void ParseConfigString(const TCHAR* Text);
	FString ParseConfigToken(const TCHAR* Text, int& Idx);

	bool HasKey(const TCHAR* Key) const;
	bool HasValue(const TCHAR* Key, const TCHAR* Value) const;

	bool TryGetValue(const TCHAR* Key, const TCHAR*& OutValue) const;
	bool TryGetValue(const TCHAR* Key, FString& OutValue) const;
	bool TryGetValue(const TCHAR* Key, FGuid& OutValue) const;
	bool TryGetValue(const TCHAR* Key, int& OutValue) const;
	bool TryGetValue(const TCHAR* Key, bool& OutValue) const;

	const TCHAR* GetValueOrDefault(const TCHAR* Key, const TCHAR* DefaultValue) const;
	FGuid GetValueOrDefault(const TCHAR* Key, const FGuid& DefaultValue) const;
	int GetValueOrDefault(const TCHAR* Key, int DefaultValue) const;
	bool GetValueOrDefault(const TCHAR* Key, bool DefaultValue) const;

	void SetValue(const TCHAR* Key, const TCHAR* Value);
	void SetValue(const TCHAR* Key, const FGuid& Value);
	void SetValue(const TCHAR* Key, int Value);
	void SetValue(const TCHAR* Key, bool Value);

	void SetDefaults(const FCustomConfigObject& Other);
	void AddOverrides(const FCustomConfigObject& Object, const FCustomConfigObject* DefaultObject);

	FString ToString(const FCustomConfigObject* BaseObject) const;
	FString ToString() const;
};

struct FCustomConfigSection
{
	const FString Name;
	TMap<FString, FString> Pairs;

	FCustomConfigSection(FString&& InName);
	void Clear();
	void SetValue(const TCHAR* Key, int Value);
	void SetValue(const TCHAR* Key, bool Value);
	void SetValue(const TCHAR* Key, const TCHAR* Value);
	void SetValues(const TCHAR* Key, const TArray<FString>& Values);
	void SetValues(const TCHAR* Key, const TArray<FGuid>& Values);
	void SetValues(const TCHAR* Key, const TArray<int>& Values);
	void AppendValue(const TCHAR* Key, const TCHAR* Value);
	void RemoveValue(const TCHAR* Key);
	int GetValue(const TCHAR* Key, int DefaultValue) const;
	bool GetValue(const TCHAR* Key, bool DefaultValue) const;
	const TCHAR* GetValue(const TCHAR* Key, const TCHAR* DefaultValue = nullptr) const;

	bool TryGetValue(const TCHAR* Key, FString& OutValue) const;
	bool TryGetValue(const TCHAR* Key, int32& OutValue) const;
	bool TryGetValues(const TCHAR* Key, TArray<FString>& Values) const;
	bool TryGetValues(const TCHAR* Key, TArray<FGuid>& Values) const;
	bool TryGetValues(const TCHAR* Key, TArray<int>& Values) const;
};

struct FCustomConfigFile
{
	TArray<TSharedPtr<FCustomConfigSection>> Sections;

	FCustomConfigFile();
	void Load(const TCHAR* FileName);
	void Parse(const TArray<FString>& Lines);
	void Save(const TCHAR* FileName);
	TSharedPtr<FCustomConfigSection> FindSection(const TCHAR* Name);
	TSharedPtr<const FCustomConfigSection> FindSection(const TCHAR* Name) const;
	TSharedRef<FCustomConfigSection> FindOrAddSection(const TCHAR* Name);

	void SetValue(const TCHAR* Key, int Value);
	void SetValue(const TCHAR* Key, bool Value);
	void SetValue(const TCHAR* Key, const TCHAR* Value);

	void SetValues(const TCHAR* Key, const TArray<FString>& Values);

	bool GetValue(const TCHAR* Key, bool DefaultValue) const;
	int32 GetValue(const TCHAR* Key, int32 DefaultValue) const;
	const TCHAR* GetValue(const TCHAR* Key, const TCHAR* DefaultValue) const;

	bool TryGetValue(const TCHAR* Key, FString& OutValue) const;
	bool TryGetValue(const TCHAR* Key, int32& OutValue) const;
	bool TryGetValues(const TCHAR* Key, TArray<FString>& Values) const;
	bool TryGetValues(const TCHAR* Key, TArray<FGuid>& Value) const;
};

} // namespace UGSCore
