// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetectProjectSettingsTask.h"
#include "OutputAdapters.h"
#include "Utility.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "UnrealGameSync"

namespace UGSCore
{

FDetectProjectSettingsTask::FDetectProjectSettingsTask(TSharedRef<FPerforceConnection> InPerforce, const FString& InNewSelectedFileName, TSharedRef<FLineBasedTextWriter> InLog)
	: NewSelectedFileName(InNewSelectedFileName)
	, Perforce(MoveTemp(InPerforce))
	, Log(MoveTemp(InLog))
{
}

FDetectProjectSettingsTask::~FDetectProjectSettingsTask()
{
}

TSharedRef<FModalTaskResult> FDetectProjectSettingsTask::Run(FEvent* AbortEvent)
{
	// TODO enable these if we want
	//try
	{
		return RunInternal(AbortEvent);
	}
	//catch(FAbortException)
	{
		//return FModalTaskResult::Aborted();
	}
}

TSharedRef<FModalTaskResult> FDetectProjectSettingsTask::RunInternal(FEvent* AbortEvent)
{
	FString PrevDirectory = FPlatformProcess::GetCurrentWorkingDirectory();

	// Get the Perforce server info
	TSharedPtr<FPerforceInfoRecord> PerforceInfo;
	if(!Perforce->Info(PerforceInfo, AbortEvent, Log.Get()))
	{
		return FModalTaskResult::Failure(LOCTEXT("NoPerforceInfo", "Couldn't get Perforce server info"));
	}
	if(PerforceInfo->UserName.Len() == 0)
	{
		return FModalTaskResult::Failure(LOCTEXT("MissingUserNameInP4Info", "Missing user name in call to p4 info"));
	}
	if(PerforceInfo->HostName.Len() == 0)
	{
		return FModalTaskResult::Failure(LOCTEXT("MissingHostNameInP4Info", "Missing host name in call to p4 info"));
	}
	ServerTimeZone = PerforceInfo->ServerTimeZone;

	// Find all the clients on this machine
	Log->Logf(TEXT("Enumerating clients on local machine..."));
	TArray<FPerforceClientRecord> Clients;
	if(!Perforce->FindClients(Clients, PerforceInfo->UserName, AbortEvent, Log.Get()))
	{
		return FModalTaskResult::Failure(LOCTEXT("NoClientsForHost", "Couldn't find any clients for this host."));
	}

	// Find any clients which are valid. If this is not exactly one, we should fail.
	TArray<TSharedRef<FPerforceConnection>> CandidateClients;
	for(const FPerforceClientRecord& Client : Clients)
	{
		// Make sure the client is well formed
		if(Client.Name.Len() > 0 && (Client.Host.Len() > 0 || Client.Owner.Len() > 0) && Client.Root.Len() > 0)
		{
			// Require either a username or host name match
			if((Client.Host.Len() == 0 || Client.Host == PerforceInfo->HostName) && (Client.Owner.Len() == 0 || Client.Owner == PerforceInfo->UserName))
			{
				if(!FUtility::IsFileUnderDirectory(*NewSelectedFileName, *Client.Root))
				{
					Log->Logf(TEXT("Rejecting %s due to root mismatch (%s)"), *Client.Name, *Client.Root);
					continue;
				}

				TSharedRef<FPerforceConnection> CandidateClient = Perforce->OpenClient(Client.Name);

				TSharedPtr<FPerforceWhereRecord> WhereRecord;
				if (!CandidateClient->Where(NewSelectedFileName, WhereRecord, AbortEvent, Log.Get()))
				{
					Log->Logf(TEXT("Rejecting %s due to file not existing in workspace"), *Client.Name);
					continue;
				}

				TArray<FPerforceFileRecord> Records;
				if(!CandidateClient->Stat(NewSelectedFileName, Records, AbortEvent, Log.Get()))
				{
					Log->Logf(TEXT("Rejecting %s due to %s not in depot"), *Client.Name, *NewSelectedFileName);
					continue;
				}

				Records.RemoveAll([](const FPerforceFileRecord& Record){ return !Record.IsMapped; });
				if(Records.Num() != 1)
				{
					Log->Logf(TEXT("Rejecting %s due to %d matching records"), *Client.Name, Records.Num());
					continue;
				}

				Log->Logf(TEXT("Found valid client %s"), *Client.Name);
				CandidateClients.Add(CandidateClient);
			}
		}
	}

	// Check there's only one client
	if(CandidateClients.Num() == 0)
	{
		return FModalTaskResult::Failure(FText::Format(LOCTEXT("CouldNotFindWorkspace", "Couldn't find any Perforce workspace containing {0}."), FText::FromString(NewSelectedFileName)));
	}
	else if(CandidateClients.Num() > 1)
	{
		FString ClientList;
		for(const TSharedRef<FPerforceConnection>& CandidateClient : CandidateClients)
		{
			ClientList += TEXT("\n");
			ClientList += CandidateClient->ClientName;
		}
		return FModalTaskResult::Failure(FText::Format(LOCTEXT("FoundMultipleWorkspaces", "Found multiple workspaces containing {0}:\n{1}\n\nCannot determine which to use."), FText::FromString(FPaths::GetCleanFilename(NewSelectedFileName)), FText::FromString(ClientList)));
	}

	// Take the client we've chosen
	PerforceClient = CandidateClients[0];

	// Get the client path for the project file
	if(!PerforceClient->ConvertToClientPath(NewSelectedFileName, NewSelectedClientFileName, AbortEvent, Log.Get()))
	{
		return FModalTaskResult::Failure(FText::Format(LOCTEXT("CouldNotGetClientPath", "Couldn't get client path for {0}"), FText::FromString(NewSelectedFileName)));
	}

	// Figure out where the engine is in relation to it
	for(int EndIdx = NewSelectedClientFileName.Len() - 1;;EndIdx--)
	{
		if(EndIdx < 2)
		{
			return FModalTaskResult::Failure(FText::Format(LOCTEXT("CouldNotFindEngine", "Could not find engine in Perforce relative to project path ({0})"), FText::FromString(NewSelectedClientFileName)));
		}
		if(NewSelectedClientFileName[EndIdx] == '/')
		{
			bool bFileExists;
			if(PerforceClient->FileExists(NewSelectedClientFileName.Mid(0, EndIdx) + "/Engine/Source/UnrealEditor.Target.cs", bFileExists, AbortEvent, Log.Get()) && bFileExists)
			{
				BranchClientPath = NewSelectedClientFileName.Mid(0, EndIdx);
				break;
			}
		}
	}
	Log->Logf(TEXT("Found branch root at %s"), *BranchClientPath);

	// Get the local branch root
	FString BaseEditorTargetPath;
	if(!PerforceClient->ConvertToLocalPath(BranchClientPath + TEXT("/Engine/Source/UnrealEditor.Target.cs"), BaseEditorTargetPath, AbortEvent, Log.Get()))
	{
		return FModalTaskResult::Failure(LOCTEXT("CouldNotGetEditorTarget", "Couldn't get local path for editor target file"));
	}
	BranchDirectoryName = FPaths::ConvertRelativePathToFull(FPaths::GetPath(FPaths::GetPath(FPaths::GetPath(BaseEditorTargetPath))));
	FPaths::MakePlatformFilename(BranchDirectoryName);

	// Find the editor target for this project
	if(NewSelectedFileName.EndsWith(TEXT(".uproject")))
	{
		TArray<FPerforceFileRecord> Files;
		if(PerforceClient->FindFiles(FPerforceUtils::GetClientOrDepotDirectoryName(*NewSelectedClientFileName) + TEXT("/Source/*Editor.Target.cs"), Files, AbortEvent, Log.Get()) && Files.Num() >= 1)
		{
			const FPerforceFileRecord* ValidFile = nullptr;
			for(const FPerforceFileRecord& File : Files)
			{
				if(File.Action.Len() == 0 || !File.Action.Contains(TEXT("delete")))
				{
					int LastSlashIdx;
					if(File.DepotPath.FindLastChar('/', LastSlashIdx))
					{
						// Grab the <Editor>.Target name
						NewProjectEditorTarget = FPaths::GetBaseFilename(File.DepotPath.Mid(LastSlashIdx + 1));

						// Remove the .Target
						NewProjectEditorTarget = FPaths::GetBaseFilename(NewProjectEditorTarget);

						Log->Logf(TEXT("Using %s as editor target name (from %s)"), *NewProjectEditorTarget, *File.DepotPath);
						break;
					}
				}
			}
			if(NewProjectEditorTarget.Len() == 0)
			{
				Log->Logf(TEXT("Couldn't find any non-deleted editor targets for this project."));
			}
		}
		else
		{
			Log->Logf(TEXT("Couldn't find any editor targets for this project."));
		}
	}

	// Get a unique name for the project that's selected. For regular branches, this can be the depot path. For streams, we want to include the stream name to encode imports.
	if(PerforceClient->GetActiveStream(StreamName, AbortEvent, Log.Get()))
	{
		FString ExpectedPrefix = FString::Printf(TEXT("//%s/"), *PerforceClient->ClientName);
		if(!NewSelectedClientFileName.StartsWith(ExpectedPrefix))
		{
			return FModalTaskResult::Failure(FText::Format(LOCTEXT("UnexpectedClientPath", "Unexpected client path; expected '{0}' to begin with '{1}'"), FText::FromString(NewSelectedClientFileName), FText::FromString(ExpectedPrefix)));
		}
		FString StreamPrefix;
		if(!TryGetStreamPrefix(PerforceClient.ToSharedRef(), StreamName, AbortEvent, Log.Get(), StreamPrefix))
		{
			return FModalTaskResult::Failure(FText::Format(LOCTEXT("FailedToGetStreamInfo", "Failed to get stream info for {0}"), FText::FromString(StreamName)));
		}
		NewSelectedProjectIdentifier = FString::Printf(TEXT("%s/%s"), *StreamPrefix, *NewSelectedClientFileName.Mid(ExpectedPrefix.Len()));
	}
	else
	{
		if(!PerforceClient->ConvertToDepotPath(NewSelectedClientFileName, NewSelectedProjectIdentifier, AbortEvent, Log.Get()))
		{
			return FModalTaskResult::Failure(FText::Format(LOCTEXT("CouldNotGetDepotPath", "Couldn't get depot path for {0}"), FText::FromString(NewSelectedFileName)));
		}
	}

	// Read the project logo
	if(NewSelectedFileName.EndsWith(TEXT(".uproject")))
	{
		/*
		string LogoFileName = Path.Combine(Path.GetDirectoryName(NewSelectedFileName), "Build", "UnrealGameSync.png");
		if(File.Exists(LogoFileName))
		{
			try
			{
				// Duplicate the image, otherwise we'll leave the file locked
				using(Image Image = Image.FromFile(LogoFileName))
				{
					ProjectLogo = new Bitmap(Image);
				}
			}
			catch
			{
				ProjectLogo = null;
			}
		}*/
	}

	// Succeeed!
	return FModalTaskResult::Success();
}

bool FDetectProjectSettingsTask::TryGetStreamPrefix(TSharedRef<FPerforceConnection> Perforce, const FString& StreamName, FEvent* AbortEvent, FLineBasedTextWriter& Log, FString& OutStreamPrefix)
{
	FString CurrentStreamName = StreamName;
	for(;;)
	{
		TSharedPtr<FPerforceSpec> StreamSpec;
		if(!Perforce->TryGetStreamSpec(CurrentStreamName, StreamSpec, AbortEvent, Log))
		{
			return false;
		}
		if(StreamSpec->GetField(TEXT("Type")) != FString(TEXT("virtual")))
		{
			OutStreamPrefix = CurrentStreamName;
			return true;
		}
		CurrentStreamName = StreamSpec->GetField(TEXT("Parent"));
	}
}

#undef LOCTEXT_NAMESPACE

} // namespace UGSCore
