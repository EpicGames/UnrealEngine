// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ModalTask.h"
#include "Perforce.h"
#include "OutputAdapters.h"

namespace UGSCore
{

class FDetectProjectSettingsTask : public IModalTask
{
public:
	const FString NewSelectedFileName;
	TSharedRef<FPerforceConnection> Perforce;
	TSharedPtr<FPerforceConnection> PerforceClient;
	FString NewSelectedProjectIdentifier;
	FString NewProjectEditorTarget;
	FString BranchClientPath;
	FString BranchDirectoryName;
	FString NewSelectedClientFileName;
	FString StreamName;
//	public Image ProjectLogo;
	FTimespan ServerTimeZone;
//	TextWriter Log;

	FDetectProjectSettingsTask(TSharedRef<FPerforceConnection> InPerforce, const FString& InNewSelectedFileName, TSharedRef<FLineBasedTextWriter> InLog);
	~FDetectProjectSettingsTask();

	virtual TSharedRef<FModalTaskResult> Run(FEvent* AbortEvent) override;

private:
	TSharedRef<FLineBasedTextWriter> Log;

	TSharedRef<FModalTaskResult> RunInternal(FEvent* AbortEvent);
	static bool TryGetStreamPrefix(TSharedRef<FPerforceConnection> Perforce, const FString& StreamName, FEvent* AbortEvent, FLineBasedTextWriter& Log, FString& OutStreamPrefix);
};

} // namespace UGSCore
