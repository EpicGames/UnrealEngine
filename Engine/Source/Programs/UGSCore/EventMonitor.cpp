// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventMonitor.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeLock.h"

namespace UGSCore
{

//// FEventData ////

FEventData::FEventData(int32 InChangeNumber, const FString& InUserName, EEventType InType, const FString& InProject)
	: FEventData(INT64_MAX, InChangeNumber, InUserName, InType, InProject)
{
}

FEventData::FEventData(int64 InId, int32 InChangeNumber, const FString& InUserName, EEventType InType, const FString& InProject)
	: Id(InId)
	, ChangeNumber(InChangeNumber)
	, UserName(InUserName)
	, Type(InType)
	, Project(InProject)
{
}

//// FCommentData ////

FCommentData::FCommentData(int32 InChangeNumber, const FString& InUserName, const FString& InText, const FString& InProject)
	: FCommentData(INT64_MAX, InChangeNumber, InUserName, InText, InProject)
{
}

FCommentData::FCommentData(int64 InId, int32 InChangeNumber, const FString& InUserName, const FString& InText, const FString& InProject)
	: Id(InId)
	, ChangeNumber(InChangeNumber)
	, UserName(InUserName)
	, Text(InText)
	, Project(InProject)
{
}

//// FEventSummary ////

FEventSummary::FEventSummary(int InChangeNumber)
	: ChangeNumber(InChangeNumber)
	, Verdict(EReviewVerdict::Unknown)
{
}

//// FEventMonitor ////

FEventMonitor::FEventMonitor(const FString& InSqlConnectionString, const FString& InProject, const FString& InCurrentUserName, const FString& InLogFileName)
	: SqlConnectionString(InSqlConnectionString)
	, Project(InProject)
	, CurrentUserName(InCurrentUserName)
	, WorkerThread(nullptr)
	, RefreshEvent(FPlatformProcess::GetSynchEventFromPool())
	, LogWriter(*InLogFileName, 128 * 1024)
	, LastEventId(0)
	, LastCommentId(0)
	, LastBuildId(0)
	, bDisposing(false)
	, bUpdateActiveInvestigations(true)
{
	if(SqlConnectionString.Len() == 0)
	{
		LastStatusMessage = TEXT("Database functionality disabled due to empty SqlConnectionString.");
	}
	else
	{
		LogWriter.Logf(TEXT("Using connection string: %s"), *SqlConnectionString);
	}
}

FEventMonitor::~FEventMonitor()
{
	bDisposing = true;
	if(WorkerThread != nullptr)
	{
		RefreshEvent->Trigger();
		WorkerThread->WaitForCompletion();
		WorkerThread = nullptr;
	}
	FPlatformProcess::ReturnSynchEventToPool(RefreshEvent);
}

void FEventMonitor::Start()
{
	check(WorkerThread == nullptr);
	WorkerThread = FRunnableThread::Create(this, TEXT("Event Monitor"));
}

void FEventMonitor::FilterChanges(const TArray<int>& ChangeNumbers)
{
	// Build a lookup for all the change numbers
	FilterChangeNumbers.Empty();
	FilterChangeNumbers.Append(ChangeNumbers);

	// Clear out the list of active users for each review we have
	UserNameToLastSyncEvent.Empty();
	for(const TTuple<int, TSharedRef<FEventSummary>>& Pair : ChangeNumberToSummary)
	{
		Pair.Value->CurrentUsers.Empty();
	}

	// Add all the user reviews back in again
	for(const TTuple<int, TSharedRef<FEventSummary>>& Pair : ChangeNumberToSummary)
	{
		const FEventSummary& Summary = *Pair.Value;
		for(const TSharedRef<FEventData>& SyncEvent : Summary.SyncEvents)
		{
			ApplyFilteredUpdate(SyncEvent);
		}
	}
}

FString FEventMonitor::GetLastStatusMessage() const
{
	FScopeLock Lock(&CriticalSection);
	return LastStatusMessage;
}

void FEventMonitor::ApplyUpdates()
{
	// Create local copies of all the incoming data
	CriticalSection.Lock();

	TArray<TSharedRef<FEventData>> LocalIncomingEvents; 
	Exchange(LocalIncomingEvents, IncomingEvents);

	TArray<TSharedRef<FBuildData>> LocalIncomingBuilds;
	Exchange(LocalIncomingBuilds, IncomingBuilds);

	TArray<TSharedRef<FCommentData>> LocalIncomingComments;
	Exchange(LocalIncomingComments, IncomingComments);

	CriticalSection.Unlock();

	// Apply all the updates
	for(const TSharedRef<FEventData>& Event : LocalIncomingEvents)
	{
		ApplyEventUpdate(Event);
	}

	for(const TSharedRef<FBuildData>& Build : LocalIncomingBuilds)
	{
		ApplyBuildUpdate(Build);
	}

	for(const TSharedRef<FCommentData>& Comment : LocalIncomingComments)
	{
		ApplyCommentUpdate(Comment);
	}
}

void FEventMonitor::PostEvent(int ChangeNumber, EEventType Type)
{
	if(SqlConnectionString.Len() > 0)
	{
		TSharedRef<FEventData> Event(new FEventData(ChangeNumber, CurrentUserName, Type, Project));

		CriticalSection.Lock();
		OutgoingEvents.Add(Event);
		CriticalSection.Unlock();

		ApplyEventUpdate(Event);

		RefreshEvent->Trigger();
	}
}

void FEventMonitor::PostComment(int ChangeNumber, const FString& Text)
{
	if(SqlConnectionString.Len() > 0)
	{
		TSharedRef<FCommentData> Comment(new FCommentData(ChangeNumber, CurrentUserName, Text, Project));

		CriticalSection.Lock();
		OutgoingComments.Add(Comment);
		CriticalSection.Unlock();

		ApplyCommentUpdate(Comment);

		RefreshEvent->Trigger();
	}
}

bool FEventMonitor::TryGetCommentByCurrentUser(int ChangeNumber, FString& OutCommentText) const
{
	TSharedPtr<FEventSummary> Summary;
	if(TryGetSummaryForChange(ChangeNumber, Summary))
	{
		for(const TSharedRef<FCommentData>& Comment : Summary->Comments)
		{
			if(Comment->UserName == CurrentUserName && Comment->Text.Len() > 0)
			{
				OutCommentText = Comment->Text;
				return true;
			}
		}
	}
	return false;
}

bool FEventMonitor::TryGetReviewByCurrentUser(int ChangeNumber, TSharedPtr<FEventData>& OutReview) const
{
	TSharedPtr<FEventSummary> Summary;
	if(TryGetSummaryForChange(ChangeNumber, Summary))
	{
		for(const TSharedRef<FEventData>& Event : Summary->Reviews)
		{
			if(Event->UserName == CurrentUserName)
			{
				OutReview = Event;
				return true;
			}
		}
	}
	return false;
}

bool FEventMonitor::TryGetSummaryForChange(int ChangeNumber, TSharedPtr<FEventSummary>& OutSummary) const
{
	const TSharedRef<FEventSummary>* Summary = ChangeNumberToSummary.Find(ChangeNumber);
	if(Summary != nullptr)
	{
		OutSummary = *Summary;
		return true;
	}
	return false;
}

bool FEventMonitor::IsReview(EEventType Type)
{
	return IsPositiveReview(Type) || IsNegativeReview(Type) || Type == EEventType::Unknown;
}

bool FEventMonitor::IsPositiveReview(EEventType Type)
{
	return Type == EEventType::Good || Type == EEventType::Compiles;
}

bool FEventMonitor::IsNegativeReview(EEventType Type)
{
	return Type == EEventType::DoesNotCompile || Type == EEventType::Bad;
}

bool FEventMonitor::WasSyncedByCurrentUser(int ChangeNumber) const
{
	TSharedPtr<FEventSummary> Summary;
	if(TryGetSummaryForChange(ChangeNumber, Summary))
	{
		for(const TSharedRef<FEventData>& SyncEvent : Summary->SyncEvents)
		{
			if(SyncEvent->Type == EEventType::Syncing && SyncEvent->UserName == CurrentUserName)
			{
				return true;
			}
		}
	}
	return false;
}

void FEventMonitor::StartInvestigating(int ChangeNumber)
{
	PostEvent(ChangeNumber, EEventType::Investigating);
}

void FEventMonitor::FinishInvestigating(int ChangeNumber)
{
	PostEvent(ChangeNumber, EEventType::Resolved);
}

bool FEventMonitor::IsUnderInvestigation(int ChangeNumber)
{
	UpdateActiveInvestigations();

	for(const TSharedRef<FEventData>& Investigation : ActiveInvestigations)
	{
		if(Investigation->ChangeNumber <= ChangeNumber)
		{
			return true;
		}
	}

	return false;
}

bool FEventMonitor::IsUnderInvestigationByCurrentUser(int ChangeNumber)
{
	UpdateActiveInvestigations();

	for(const TSharedRef<FEventData>& Investigation : ActiveInvestigations)
	{
		if(Investigation->ChangeNumber <= ChangeNumber && Investigation->UserName == CurrentUserName)
		{
			return true;
		}
	}

	return false;
}

TArray<FString> FEventMonitor::GetInvestigatingUsers(int ChangeNumber)
{
	UpdateActiveInvestigations();

	TArray<FString> UserNames;
	for(const TSharedRef<FEventData>& Investigation : ActiveInvestigations)
	{
		if(Investigation->ChangeNumber <= ChangeNumber)
		{
			UserNames.Add(Investigation->UserName);
		}
	}
	return UserNames;
}

int FEventMonitor::GetInvestigationStartChangeNumber(int LastChangeNumber)
{
	UpdateActiveInvestigations();

	int StartChangeNumber = -1;
	for(const TSharedRef<FEventData>& ActiveInvestigation : ActiveInvestigations)
	{
		if(ActiveInvestigation->UserName == CurrentUserName)
		{
			if(ActiveInvestigation->ChangeNumber <= LastChangeNumber && (StartChangeNumber == -1 || ActiveInvestigation->ChangeNumber < StartChangeNumber))
			{
				StartChangeNumber = ActiveInvestigation->ChangeNumber;
			}
		}
	}
	return StartChangeNumber;
}

TSharedRef<FEventSummary> FEventMonitor::FindOrAddSummary(int ChangeNumber)
{
	TSharedRef<FEventSummary>* SummaryPtr = ChangeNumberToSummary.Find(ChangeNumber);
	if(SummaryPtr == nullptr)
	{
		SummaryPtr = &(ChangeNumberToSummary.Add(ChangeNumber, TSharedRef<FEventSummary>(new FEventSummary(ChangeNumber))));
	}
	return *SummaryPtr;
}

void FEventMonitor::ApplyEventUpdate(const TSharedRef<FEventData>& Event)
{
	TSharedRef<FEventSummary> Summary = FindOrAddSummary(Event->ChangeNumber);
	if(Event->Type == EEventType::Starred || Event->Type == EEventType::Unstarred)
	{
		// If it's a star or un-star review, process that separately
		if(!Summary->LastStarReview.IsValid() || Event->Id > Summary->LastStarReview->Id)
		{
			Summary->LastStarReview = Event;
		}
	}
	else if(Event->Type == EEventType::Investigating || Event->Type == EEventType::Resolved)
	{
		// Insert it sorted in the investigation list
		int InsertIdx = 0;
		while(InsertIdx < InvestigationEvents.Num() && InvestigationEvents[InsertIdx]->Id < Event->Id)
		{
			InsertIdx++;
		}
		if(InsertIdx == InvestigationEvents.Num() || InvestigationEvents[InsertIdx]->Id != Event->Id)
		{
			InvestigationEvents.Insert(Event, InsertIdx);
		}
		bUpdateActiveInvestigations = true;
	}
	else if(Event->Type == EEventType::Syncing)
	{
		Summary->SyncEvents.RemoveAll([&Event](const TSharedRef<FEventData>& OtherEvent){ return Event->UserName == OtherEvent->UserName; });
		Summary->SyncEvents.Add(Event);
		ApplyFilteredUpdate(Event);
	}
	else if(IsReview(Event->Type))
	{
		// Try to find an existing review by this user. If we already have a newer review, ignore this one. Otherwise remove it.
		for(int Idx = 0; Idx < Summary->Reviews.Num(); Idx++)
		{
			const TSharedRef<FEventData>& ExistingReview = Summary->Reviews[Idx];
			if(ExistingReview->UserName == Event->UserName)
			{
				if(ExistingReview->Id > Event->Id)
				{
					return;
				}
				Summary->Reviews.RemoveAt(Idx);
				break;
			}
		}

		// Add the new review, and find the new verdict for this change
		Summary->Reviews.Add(Event);
		Summary->Verdict = GetVerdict(Summary->Reviews, Summary->Builds);
	}
	else
	{
		// Unknown type
	}
}

void FEventMonitor::ApplyBuildUpdate(const TSharedRef<FBuildData>& Build)
{
	TSharedRef<FEventSummary> Summary = FindOrAddSummary(Build->ChangeNumber);

	for(int Idx = 0; Idx < Summary->Builds.Num(); Idx++)
	{
		const TSharedRef<FBuildData>& ExistingBuild = Summary->Builds[Idx];
		if(ExistingBuild->ChangeNumber == Build->ChangeNumber && ExistingBuild->BuildType == Build->BuildType)
		{
			if(ExistingBuild->Id > Build->Id)
			{
				return;
			}
			Summary->Builds.RemoveAt(Idx);
			break;
		}
	}

	Summary->Builds.Add(Build);
	Summary->Verdict = GetVerdict(Summary->Reviews, Summary->Builds);
}

void FEventMonitor::ApplyCommentUpdate(const TSharedRef<FCommentData>& Comment)
{
//	TSharedRef<FEventSummary> Summary = FindOrAddSummary(Comment->ChangeNumber);
//	if(String.Compare(Comment.UserName, CurrentUserName, true) == 0 && Summary.Comments.Count > 0 && Summary.Comments.Last().Id == long.MaxValue)
//	{
//		// This comment was added by PostComment(), to mask the latency of a round trip to the server. Remove it now we have the sorted comment.
//		Summary.Comments.RemoveAt(Summary.Comments.Count - 1);
//	}
//	AddPerUserItem(Summary.Comments, Comment, x => x.Id, x => x.UserName);
}
/*
static bool AddPerUserItem<T>(List<T> Items, T NewItem, Func<T, long> IdSelector, Func<T, string> UserSelector)
{
	int InsertIdx = Items.Count;

	for(; InsertIdx > 0 && IdSelector(Items[InsertIdx - 1]) >= IdSelector(NewItem); InsertIdx--)
	{
		if(String.Compare(UserSelector(Items[InsertIdx - 1]), UserSelector(NewItem), true) == 0)
		{
			return false;
		}
	}

	Items.Insert(InsertIdx, NewItem);

	for(; InsertIdx > 0; InsertIdx--)
	{
		if(String.Compare(UserSelector(Items[InsertIdx - 1]), UserSelector(NewItem), true) == 0)
		{
			Items.RemoveAt(InsertIdx - 1);
		}
	}

	return true;
}
*/
EReviewVerdict FEventMonitor::GetVerdict(const TArray<TSharedRef<FEventData>>& Events, const TArray<TSharedRef<FBuildData>>& Builds)
{
	int NumPositiveReviews = 0;
	int NumNegativeReviews = 0;

	int NumCompiles = 0;
	int NumFailedCompiles = 0;

	for(const TSharedRef<FEventData>& Event : Events)
	{
		if(Event->Type == EEventType::Good)
		{
			NumPositiveReviews++;
		}
		else if(Event->Type == EEventType::Bad)
		{
			NumNegativeReviews++;
		}
		else if(Event->Type == EEventType::Compiles)
		{
			NumCompiles++;
		}
		else if(Event->Type == EEventType::DoesNotCompile)
		{
			NumFailedCompiles++;
		}
	}

	if(NumPositiveReviews > 0 || NumNegativeReviews > 0)
	{
		return GetVerdict(NumPositiveReviews, NumNegativeReviews);
	}

	if(NumCompiles > 0 || NumFailedCompiles > 0)
	{
		return GetVerdict(NumCompiles, NumFailedCompiles);
	}

	int NumBuilds = 0;
	int NumFailedBuilds = 0;

	for(const TSharedRef<FBuildData>& Build : Builds)
	{
		if(Build->BuildType == TEXT("Editor"))
		{
			if(Build->IsSuccess())
			{
				NumBuilds++;
			}
			if(Build->IsFailure())
			{
				NumFailedBuilds++;
			}
		}
	}

	if(NumBuilds > 0 || NumFailedBuilds > 0)
	{
		return GetVerdict(NumBuilds, NumFailedBuilds);
	}

	return EReviewVerdict::Unknown;
}

EReviewVerdict FEventMonitor::GetVerdict(int NumPositive, int NumNegative)
{
	if(NumPositive > (int)(NumNegative * 1.5))
	{
		return EReviewVerdict::Good;
	}
	else if(NumPositive >= NumNegative)
	{
		return EReviewVerdict::Mixed;
	}
	else
	{
		return EReviewVerdict::Bad;
	}
}

void FEventMonitor::ApplyFilteredUpdate(const TSharedRef<FEventData>& Event)
{
	if(Event->Type == EEventType::Syncing && FilterChangeNumbers.Contains(Event->ChangeNumber))
	{
		// Update the active users list for this change
		TSharedRef<FEventData>* LastSync = UserNameToLastSyncEvent.Find(Event->UserName);
		if(LastSync == nullptr)
		{
			FindOrAddSummary(Event->ChangeNumber)->CurrentUsers.Add(Event->UserName);
			UserNameToLastSyncEvent[Event->UserName] = Event;
		}
		else if(Event->Id > (*LastSync)->Id)
		{
			ChangeNumberToSummary[(*LastSync)->ChangeNumber]->CurrentUsers.Remove(Event->UserName);
			FindOrAddSummary(Event->ChangeNumber)->CurrentUsers.Add(Event->UserName);
			UserNameToLastSyncEvent[Event->UserName] = Event;
		}
	}
}

uint32 FEventMonitor::Run()
{
	TArray<TSharedRef<FEventData>> LocalOutgoingEvents;
	TArray<TSharedRef<FCommentData>> LocalOutgoingComments;
	while(!bDisposing)
	{
		// If there's no connection string, just empty out the queue
		if(SqlConnectionString.Len() > 0)
		{
			// Append all the queued events to the local queue on this thread
			CriticalSection.Lock();

			LocalOutgoingEvents += OutgoingEvents;
			OutgoingEvents.Empty();

			LocalOutgoingComments += OutgoingComments;
			OutgoingComments.Empty();

			CriticalSection.Unlock();

			// Post all the reviews to the database. We don't send them out of order, so keep the review outside the queue until the next update if it fails
			while(LocalOutgoingEvents.Num() > 0 && SendEventToBackend(LocalOutgoingEvents[0]))
			{
				LocalOutgoingEvents.RemoveAt(0);
			}

			// Post all the comments to the database.
			while(LocalOutgoingComments.Num() > 0 && SendCommentToBackend(LocalOutgoingComments[0]))
			{
				LocalOutgoingComments.RemoveAt(0);
			}

			// Read all the new reviews
			ReadEventsFromBackend();

			// Send a notification that we're ready to update
//			if((IncomingEvents.Num() > 0 || IncomingBuilds.Num() > 0 || IncomingComments.Num() > 0) && OnUpdatesReady != null)
//			{
//				OnUpdatesReady();
//			}
		}

		// Wait for something else to do
		RefreshEvent->Wait(FTimespan::FromSeconds(30.0));
	}
	return 0;
}

bool FEventMonitor::SendEventToBackend(const TSharedRef<FEventData>& Event)
{
//	try
//	{
//		Stopwatch Timer = Stopwatch.StartNew();
//		LogWriter.WriteLine("Posting event... ({0}, {1}, {2})", Event.Change, Event.UserName, Event.Type);
//		using(SqlConnection Connection = new SqlConnection(SqlConnectionString))
//		{
//			Connection.Open();
//			using (SqlCommand Command = new SqlCommand("INSERT INTO dbo.UserVotes (Changelist, UserName, Verdict, Project) VALUES (@Changelist, @UserName, @Verdict, @Project)", Connection))
//			{
//				Command.Parameters.AddWithValue("@Changelist", Event.Change);
//				Command.Parameters.AddWithValue("@UserName", Event.UserName.ToString());
//				Command.Parameters.AddWithValue("@Verdict", Event.Type.ToString());
//				Command.Parameters.AddWithValue("@Project", Event.Project);
//				Command.ExecuteNonQuery();
//			}
//		}
//		LogWriter.WriteLine("Done in {0}ms.", Timer.ElapsedMilliseconds);
//		return true;
//	}
//	catch(Exception Ex)
//	{
//		LogWriter.WriteException(Ex, "Failed with exception.");
//		return false;
//	}
	return true;
}

bool FEventMonitor::SendCommentToBackend(const TSharedRef<FCommentData>& Comment)
{
//	try
//	{
//		Stopwatch Timer = Stopwatch.StartNew();
//		LogWriter.WriteLine("Posting comment... ({0}, {1}, {2}, {3})", Comment.ChangeNumber, Comment.UserName, Comment.Text, Comment.Project);
//		using(SqlConnection Connection = new SqlConnection(SqlConnectionString))
//		{
//			Connection.Open();
//			using (SqlCommand Command = new SqlCommand("INSERT INTO dbo.Comments (ChangeNumber, UserName, Text, Project) VALUES (@ChangeNumber, @UserName, @Text, @Project)", Connection))
//			{
//				Command.Parameters.AddWithValue("@ChangeNumber", Comment.ChangeNumber);
//				Command.Parameters.AddWithValue("@UserName", Comment.UserName);
//				Command.Parameters.AddWithValue("@Text", Comment.Text);
//				Command.Parameters.AddWithValue("@Project", Comment.Project);
//				Command.ExecuteNonQuery();
//			}
//		}
//		LogWriter.WriteLine("Done in {0}ms.", Timer.ElapsedMilliseconds);
//		return true;
//	}
//	catch(Exception Ex)
//	{
//		LogWriter.WriteException(Ex, "Failed with exception.");
//		return false;
//	}
	return true;
}

bool FEventMonitor::ReadEventsFromBackend()
{
	/*
	try
	{
		Stopwatch Timer = Stopwatch.StartNew();
		LogWriter.WriteLine();
		LogWriter.WriteLine("Polling for reviews at {0}...", DateTime.Now.ToString());

		if(LastEventId == 0)
		{
			using(SqlConnection Connection = new SqlConnection(SqlConnectionString))
			{
				Connection.Open();
				using (SqlCommand Command = new SqlCommand("SELECT MAX(ID) FROM dbo.UserVotes", Connection))
				{
					using (SqlDataReader Reader = Command.ExecuteReader())
					{
						while (Reader.Read())
						{
							LastEventId = Reader.GetInt64(0);
							LastEventId = Math.Max(LastEventId - 5000, 0);
							break;
						}
					}
				}
			}
		}

		using(SqlConnection Connection = new SqlConnection(SqlConnectionString))
		{
			Connection.Open();
			using (SqlCommand Command = new SqlCommand("SELECT Id, Changelist, UserName, Verdict, Project FROM dbo.UserVotes WHERE Id > @param1 ORDER BY Id", Connection))
			{
				Command.Parameters.AddWithValue("@param1", LastEventId);
				using (SqlDataReader Reader = Command.ExecuteReader())
				{
					while (Reader.Read())
					{
						EventData Review = new EventData();
						Review.Id = Reader.GetInt64(0);
						Review.Change = Reader.GetInt32(1);
						Review.UserName = Reader.GetString(2);
						Review.Project = Reader.IsDBNull(4)? null : Reader.GetString(4);
						if(Enum.TryParse(Reader.GetString(3), out Review.Type))
						{
							if(Review.Project == null || String.Compare(Review.Project, Project, true) == 0)
							{
								IncomingEvents.Enqueue(Review);
							}
							LastEventId = Math.Max(LastEventId, Review.Id);
						}
					}
				}
			}
			using(SqlCommand Command = new SqlCommand("SELECT Id, ChangeNumber, UserName, Text, Project FROM dbo.Comments WHERE Id > @param1 ORDER BY Id", Connection))
			{
				Command.Parameters.AddWithValue("@param1", LastCommentId);
				using (SqlDataReader Reader = Command.ExecuteReader())
				{
					while (Reader.Read())
					{
						CommentData Comment = new CommentData();
						Comment.Id = Reader.GetInt32(0);
						Comment.ChangeNumber = Reader.GetInt32(1);
						Comment.UserName = Reader.GetString(2);
						Comment.Text = Reader.GetString(3);
						Comment.Project = Reader.GetString(4);
						if(Comment.Project == null || String.Compare(Comment.Project, Project, true) == 0)
						{
							IncomingComments.Enqueue(Comment);
						}
						LastCommentId = Math.Max(LastCommentId, Comment.Id);
					}
				}
			}
			using(SqlCommand Command = new SqlCommand("SELECT Id, ChangeNumber, BuildType, Result, Url, Project FROM dbo.CIS WHERE Id > @param1 ORDER BY Id", Connection))
			{
				Command.Parameters.AddWithValue("@param1", LastBuildId);
				using (SqlDataReader Reader = Command.ExecuteReader())
				{
					while (Reader.Read())
					{
						BuildData Build = new BuildData();
						Build.Id = Reader.GetInt32(0);
						Build.ChangeNumber = Reader.GetInt32(1);
						Build.BuildType = Reader.GetString(2).TrimRight();
						if(Enum.TryParse(Reader.GetString(3).TrimRight(), true, out Build.Result))
						{
							Build.Url = Reader.GetString(4);
							Build.Project = Reader.IsDBNull(5)? null : Reader.GetString(5);
							if(Build.Project == null || String.Compare(Build.Project, Project, true) == 0 || MatchesWildcard(Build.Project, Project))
							{
								IncomingBuilds.Enqueue(Build);
							}
						}
						LastBuildId = Math.Max(LastBuildId, Build.Id);
					}
				}
			}
		}
		LastStatusMessage = String.Format("Last update took {0}ms", Timer.ElapsedMilliseconds);
		LogWriter.WriteLine("Done in {0}ms.", Timer.ElapsedMilliseconds);
		return true;
	}
	catch(Exception Ex)
	{
		LogWriter.WriteException(Ex, "Failed with exception.");
		LastStatusMessage = String.Format("Last update failed: ({0})", Ex.ToString());
		return false;
	}
	*/
	return true;
}

bool FEventMonitor::MatchesWildcard(const FString& Wildcard, const FString& Project)
{
	return Wildcard.EndsWith("...") && Project.StartsWith(Wildcard.Mid(0, Wildcard.Len() - 3));
}

void FEventMonitor::UpdateActiveInvestigations()
{
	if(bUpdateActiveInvestigations)
	{
		// Insert investigation events into the active list, sorted by change number. Remove 
		ActiveInvestigations.Empty();
		for(const TSharedRef<FEventData>& InvestigationEvent : InvestigationEvents)
		{
			if(FilterChangeNumbers.Contains(InvestigationEvent->ChangeNumber))
			{
				if(InvestigationEvent->Type == EEventType::Investigating)
				{
					int InsertIdx = 0;
					while(InsertIdx < ActiveInvestigations.Num() && ActiveInvestigations[InsertIdx]->ChangeNumber > InvestigationEvent->ChangeNumber)
					{
						InsertIdx++;
					}
					ActiveInvestigations.Insert(InvestigationEvent, InsertIdx);
				}
				else
				{
					ActiveInvestigations.RemoveAll([&InvestigationEvent](const TSharedRef<FEventData>& Event){ return Event->UserName == InvestigationEvent->UserName && Event->ChangeNumber <= InvestigationEvent->ChangeNumber; });
				}
			}
		}

		// Remove any duplicate users
		for(int Idx = 0; Idx < ActiveInvestigations.Num(); Idx++)
		{
			for(int OtherIdx = 0; OtherIdx < Idx; OtherIdx++)
			{
				if(ActiveInvestigations[Idx]->UserName == ActiveInvestigations[OtherIdx]->UserName)
				{
					ActiveInvestigations.RemoveAt(Idx--);
					break;
				}
			}
		}
	}
}

} // namespace UGSCore
