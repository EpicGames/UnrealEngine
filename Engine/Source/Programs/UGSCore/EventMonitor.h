// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OutputAdapters.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"

namespace UGSCore
{

enum class EEventType
{
	Syncing,

	// Reviews
	Compiles,
	DoesNotCompile,
	Good,
	Bad,
	Unknown,
		
	// Starred builds
	Starred,
	Unstarred,

	// Investigating events
	Investigating,
	Resolved,
};

struct FEventData
{
	int64 Id;
	int32 ChangeNumber;
	FString UserName;
	EEventType Type;
	FString Project;

	FEventData(int32 InChangeNumber, const FString& InUserName, EEventType InType, const FString& InProject);
	FEventData(int64 InId, int32 InChangeNumber, const FString& InUserName, EEventType InType, const FString& InProject);
};

struct FCommentData
{
	int64 Id;
	int32 ChangeNumber;
	FString UserName;
	FString Text;
	FString Project;

	FCommentData(int32 InChangeNumber, const FString& InUserName, const FString& InText, const FString& InProject);
	FCommentData(int64 InId, int32 InChangeNumber, const FString& InUserName, const FString& InText, const FString& InProject);
};

enum class EBuildDataResult
{
	Starting,
	Failure,
	Warning,
	Success,
};

struct FBuildData
{
	long Id;
	int ChangeNumber;
	FString BuildType;
	EBuildDataResult Result;
	FString Url;
	FString Project;

	bool IsSuccess() const
	{
		return Result == EBuildDataResult::Success || Result == EBuildDataResult::Warning;
	}

	bool IsFailure() const
	{
		return Result == EBuildDataResult::Failure;
	}
};

enum class EReviewVerdict
{
	Unknown,
	Good,
	Bad,
	Mixed,
};

struct FEventSummary
{
	int ChangeNumber;
	EReviewVerdict Verdict;
	TArray<TSharedRef<FEventData>> SyncEvents;
	TArray<TSharedRef<FEventData>> Reviews;
	TArray<FString> CurrentUsers;
	TSharedPtr<FEventData> LastStarReview;
	TArray<TSharedRef<FBuildData>> Builds;
	TArray<TSharedRef<FCommentData>> Comments;

	FEventSummary(int InChangeNumber);
};

class FEventMonitor : FRunnable
{
public:
	TFunction<void()> OnUpdatesReady;

	FEventMonitor(const FString& InSqlConnectionString, const FString& InProject, const FString& InCurrentUserName, const FString& InLogFileName);
	~FEventMonitor();

	void Start();
	void FilterChanges(const TArray<int>& ChangeNumbers);

	FString GetLastStatusMessage() const;

	void ApplyUpdates();
	void PostEvent(int ChangeNumber, EEventType Type);
	void PostComment(int ChangeNumber, const FString& Text);

	bool TryGetCommentByCurrentUser(int ChangeNumber, FString& OutCommentText) const;
	bool TryGetReviewByCurrentUser(int ChangeNumber, TSharedPtr<FEventData>& OutReview) const;
	bool TryGetSummaryForChange(int ChangeNumber, TSharedPtr<FEventSummary>& OutSummary) const;

	static bool IsReview(EEventType Type);
	static bool IsPositiveReview(EEventType Type);
	static bool IsNegativeReview(EEventType Type);

	bool WasSyncedByCurrentUser(int ChangeNumber) const;

	void StartInvestigating(int ChangeNumber);
	void FinishInvestigating(int ChangeNumber);
	bool IsUnderInvestigation(int ChangeNumber);
	bool IsUnderInvestigationByCurrentUser(int ChangeNumber);
	TArray<FString> GetInvestigatingUsers(int ChangeNumber);
	int GetInvestigationStartChangeNumber(int LastChangeNumber);

private:
	const FString SqlConnectionString;
	const FString Project;
	const FString CurrentUserName;
	FRunnableThread* WorkerThread;
	mutable FCriticalSection CriticalSection;
	FEvent* RefreshEvent;
	TArray<TSharedRef<FEventData>> OutgoingEvents;
	TArray<TSharedRef<FEventData>> IncomingEvents;
	TArray<TSharedRef<FCommentData>> OutgoingComments;
	TArray<TSharedRef<FCommentData>> IncomingComments;
	TArray<TSharedRef<FBuildData>> IncomingBuilds;
	TMap<int, TSharedRef<FEventSummary>> ChangeNumberToSummary;
	TMap<FString, TSharedRef<FEventData>> UserNameToLastSyncEvent;
	FBoundedLogWriter LogWriter;
	long LastEventId;
	long LastCommentId;
	long LastBuildId;
	bool bDisposing;
	FString LastStatusMessage;
	TSet<int> FilterChangeNumbers;
	TArray<TSharedRef<FEventData>> InvestigationEvents;
	bool bUpdateActiveInvestigations;
	TArray<TSharedRef<FEventData>> ActiveInvestigations;

	TSharedRef<FEventSummary> FindOrAddSummary(int ChangeNumber);
	void ApplyEventUpdate(const TSharedRef<FEventData>& Event);
	void ApplyBuildUpdate(const TSharedRef<FBuildData>& Build);
	void ApplyCommentUpdate(const TSharedRef<FCommentData>& Comment);

	static EReviewVerdict GetVerdict(const TArray<TSharedRef<FEventData>>& Events, const TArray<TSharedRef<FBuildData>>& Builds);
	static EReviewVerdict GetVerdict(int NumPositive, int NumNegative);

	void ApplyFilteredUpdate(const TSharedRef<FEventData>& Event);
	uint32 Run();
	bool SendEventToBackend(const TSharedRef<FEventData>& Event);
	bool SendCommentToBackend(const TSharedRef<FCommentData>& Comment);
	bool ReadEventsFromBackend();

	static bool MatchesWildcard(const FString& Wildcard, const FString& Project);

	void UpdateActiveInvestigations();
};

} // namespace UGSCore
