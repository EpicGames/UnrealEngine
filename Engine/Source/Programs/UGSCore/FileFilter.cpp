// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileFilter.h"

namespace UGSCore
{

/// <summary>
/// Node within a filter tree. Each node matches a single path fragment - a folder or file name, with an include or exclude rule consisting of a sequence of nodes. 
/// </summary>
struct FFileFilter::FNode
{
	/// <summary>
	/// Node which this is parented to. Null for the root node.
	/// </summary>
	FNode* const Parent;

	/// <summary>
	/// Pattern to match for this node. May contain * or ? wildcards.
	/// </summary>
	const FString Pattern;

	/// <summary>
	/// Highest include rule number matched by this node or any child nodes.
	/// </summary>
	int MaxIncludeRuleNumber = -1;

	/// <summary>
	/// Highest exclude rule number matched by this node or any child nodes.
	/// </summary>
	int MaxExcludeRuleNumber = -1;

	/// <summary>
	/// Child branches of this node, distinct by the the pattern for each.
	/// </summary>
	TArray<FNode*> Branches;

	/// <summary>
	/// If a path matches the sequence of nodes ending in this node, the number of the rule which added it. -1 if this was not the last node in a rule.
	/// </summary>
	int RuleNumber;

	/// <summary>
	/// Whether a rule terminating in this node should include files or exclude files.
	/// </summary>
	EFileFilterType Type;

	/// <summary>
	/// Default constructor.
	/// </summary>
	FNode(FNode* InParent, const FString& InPattern)
		: Parent(InParent)
		, Pattern(InPattern)
	{
		RuleNumber = -1;
		Type = EFileFilterType::Exclude;
	}

	/// <summary>
	/// Destructor.
	/// </summary>
	~FNode()
	{
		for(FNode* Branch : Branches)
		{
			delete Branch;
			Branch = nullptr;
		}
	}

	/// <summary>
	/// Determine if the given token matches the pattern in this node
	/// </summary>
	bool IsMatch(const FString& Token) const
	{
		if(Pattern.EndsWith("."))
		{
			return !Token.Contains(TEXT(".")) && IsMatch(Token, 0, Pattern.Mid(0, Pattern.Len() - 1), 0);
		}
		else
		{
			return IsMatch(Token, 0, Pattern, 0);
		}
	}

	/// <summary>
	/// Determine if a given token matches a pattern, starting from the given positions within each string.
	/// </summary>
	static bool IsMatch(const FString& Token, int TokenIdx, const FString& Pattern, int PatternIdx)
	{
		for (; ; )
		{
			if (PatternIdx == Pattern.Len())
			{
				return (TokenIdx == Token.Len());
			}
			else if (Pattern[PatternIdx] == '*')
			{
				for (int NextTokenIdx = Token.Len(); NextTokenIdx >= TokenIdx; NextTokenIdx--)
				{
					if (IsMatch(Token, NextTokenIdx, Pattern, PatternIdx + 1))
					{
						return true;
					}
				}
				return false;
			}
			else if (TokenIdx < Token.Len() && (FChar::ToLower(Token[TokenIdx]) == FChar::ToLower(Pattern[PatternIdx]) || Pattern[PatternIdx] == '?'))
			{
				TokenIdx++;
				PatternIdx++;
			}
			else
			{
				return false;
			}
		}
	}

	/// <summary>
	/// Converts this node back to a pattern from the root
	/// </summary>
	/// <returns>Path to this node</returns>
	FString ToString() const
	{
		if(Parent == nullptr)
		{
			return TEXT("/");
		}
		else if(Parent->Parent == nullptr)
		{
			return FString(TEXT("/")) + Pattern;
		}
		else
		{
			return Parent->ToString() + FString(TEXT("/")) + Pattern;
		}
	}
};

FFileFilter::FFileFilter(EFileFilterType DefaultType)
{
	RootNode = new FNode(nullptr, "");

	DefaultNode = new FNode(RootNode, "...");
	DefaultNode->Type = DefaultType;
}

FFileFilter::FFileFilter(const FFileFilter& Other)
	: FFileFilter(Other.DefaultNode->Type)
{
	for(const FNode* OtherRule : Other.Rules)
	{
		AddRule(OtherRule->ToString(), OtherRule->Type);
	}
}

FFileFilter::~FFileFilter()
{
	delete RootNode;
}

/*
void FFileFilter::ReadRulesFromFile(string FileName, string SectionName, params string[] AllowTags)
{
	bool bInSection = false;
	foreach(string Line in File.ReadAllLines(FileName))
	{
		string TrimLine = Line.Trim();
		if(!TrimLine.StartsWith(";") && TrimLine.Length > 0)
		{
			if(TrimLine.StartsWith("["))
			{
				bInSection = (TrimLine == "[" + SectionName + "]");
			}
			else if(bInSection)
			{
				AddRule(Line, AllowTags);
			}
		}
	}
}
*/

void FFileFilter::AddRule(const FString& Pattern)
{
	if(Pattern.StartsWith(TEXT("-")))
	{
		AddRule(Pattern.Mid(1), EFileFilterType::Exclude);
	}
	else
	{
		AddRule(Pattern, EFileFilterType::Include);
	}
}

void FFileFilter::AddRule(const FString& Pattern, EFileFilterType Type)
{
	FString NormalizedPattern = Pattern;
	NormalizedPattern.ReplaceInline(TEXT("\\"), TEXT("/"));

	// We don't want a slash at the start, but if there was not one specified, it's not anchored to the root of the tree.
	if (NormalizedPattern.StartsWith(TEXT("/")))
	{
		NormalizedPattern = NormalizedPattern.Mid(1);
	}
	else if(!NormalizedPattern.StartsWith("..."))
	{
		NormalizedPattern = TEXT(".../") + NormalizedPattern;
	}

	// All directories indicate a wildcard match
	if (NormalizedPattern.EndsWith(TEXT("/")))
	{
		NormalizedPattern += TEXT("...");
	}

	// Replace any directory wildcards mid-string
	for(int Idx = 0;;)
	{
		Idx = NormalizedPattern.Find(TEXT("..."), ESearchCase::CaseSensitive, ESearchDir::FromStart, Idx);
		if(Idx == INDEX_NONE)
		{
			break;
		}

		if (Idx > 0 && NormalizedPattern[Idx - 1] != '/')
		{
			NormalizedPattern.InsertAt(Idx, TEXT("*/"));
			Idx++;
		}

		Idx += 3;

		if (Idx < NormalizedPattern.Len() && NormalizedPattern[Idx] != '/')
		{
			NormalizedPattern.InsertAt(Idx, TEXT("/*"));
			Idx += 2;
		}
	}

	// Split the pattern into fragments
	TArray<FString> BranchPatterns;
	NormalizedPattern.ParseIntoArray(BranchPatterns, TEXT("/"), false);

	// Add it into the tree
	FNode* LastNode = RootNode;
	for(const FString& BranchPattern : BranchPatterns)
	{
		FNode* NextNode = nullptr;
		for(FNode* BranchNode : LastNode->Branches)
		{
			if(BranchNode->Pattern == BranchPattern)
			{
				NextNode = BranchNode;
				break;
			}
		}
		if (NextNode == nullptr)
		{
			NextNode = new FNode(LastNode, BranchPattern);
			LastNode->Branches.Add(NextNode);
		}
		LastNode = NextNode;
	}

	// We've reached the end of the pattern, so mark it as a leaf node
	Rules.Add(LastNode);
	LastNode->RuleNumber = Rules.Num() - 1;
	LastNode->Type = Type;

	// Update the maximums along that path
	for (FNode* UpdateNode = LastNode; UpdateNode != nullptr; UpdateNode = UpdateNode->Parent)
	{
		if (Type == EFileFilterType::Include)
		{
			UpdateNode->MaxIncludeRuleNumber = LastNode->RuleNumber;
		}
		else
		{
			UpdateNode->MaxExcludeRuleNumber = LastNode->RuleNumber;
		}
	}
}

TArray<FString> TokenizePath(const FString& Path)
{
	int Idx = 0;
	while(Idx < Path.Len() && (Path[Idx] == '\\' || Path[Idx] == '/'))
	{
		Idx++;
	}

	TArray<FString> Tokens;

	int PrevIdx = Idx;
	for(; Idx < Path.Len(); Idx++)
	{
		if(Path[Idx] == '/' || Path[Idx] == '\\')
		{
			Tokens.Add(Path.Mid(PrevIdx, Idx - PrevIdx));
			PrevIdx = Idx + 1;
		}
	}
	if(Idx > PrevIdx)
	{
		Tokens.Add(Path.Mid(PrevIdx, Idx - PrevIdx));
	}

	return Tokens;
}

bool FFileFilter::Matches(const FString& FileName) const
{
	TArray<FString> Tokens = TokenizePath(FileName);

	const FNode* MatchingNode = FindMatchingNode(RootNode, Tokens, 0, DefaultNode);

	return MatchingNode->Type == EFileFilterType::Include;
}

bool FFileFilter::PossiblyMatches(const FString& FolderName) const
{
	TArray<FString> Tokens = TokenizePath(FolderName);
	Tokens.Add(TEXT(""));

	const FNode* MatchingNode = FindMatchingNode(RootNode, Tokens, 0, DefaultNode);

	return MatchingNode->Type == EFileFilterType::Include || HighestPossibleIncludeMatch(RootNode, Tokens, 0, MatchingNode->RuleNumber) > MatchingNode->RuleNumber;
}

const FFileFilter::FNode* FFileFilter::FindMatchingNode(const FNode* CurrentNode, const TArray<FString>& Tokens, int TokenIdx, const FNode* CurrentBestNode)
{
	// If we've matched all the input tokens, check if this rule is better than any other we've seen
	if (TokenIdx == Tokens.Num())
	{
		return (CurrentNode->RuleNumber > CurrentBestNode->RuleNumber) ? CurrentNode : CurrentBestNode;
	}

	// If there is no rule under the current node which is better than the current best node, early out
	if (CurrentNode->MaxIncludeRuleNumber <= CurrentBestNode->RuleNumber && CurrentNode->MaxExcludeRuleNumber <= CurrentBestNode->RuleNumber)
	{
		return CurrentBestNode;
	}

	// Test all the branches for one that matches
	const FNode* BestNode = CurrentBestNode;
	for(const FNode* Branch : CurrentNode->Branches)
	{
		if (Branch->Pattern == "...")
		{
			for (int NextTokenIdx = Tokens.Num(); NextTokenIdx >= TokenIdx; NextTokenIdx--)
			{
				BestNode = FindMatchingNode(Branch, Tokens, NextTokenIdx, BestNode);
			}
		}
		else
		{
			if (Branch->IsMatch(Tokens[TokenIdx]))
			{
				BestNode = FindMatchingNode(Branch, Tokens, TokenIdx + 1, BestNode);
			}
		}
	}
	return BestNode;
}

int FFileFilter::HighestPossibleIncludeMatch(const FNode* CurrentNode, const TArray<FString>& Tokens, int TokenIdx, int CurrentBestRuleNumber)
{
	// If we've matched all the input tokens, check if this rule is better than any other we've seen
	if (TokenIdx == Tokens.Num())
	{
		return FMath::Max(CurrentBestRuleNumber, CurrentNode->MaxIncludeRuleNumber);
	}

	// Test all the branches for one that matches
	int BestRuleNumber = CurrentBestRuleNumber;
	if (CurrentNode->MaxIncludeRuleNumber > BestRuleNumber)
	{
		for(const FNode* Branch : CurrentNode->Branches)
		{
			if (Branch->Pattern == "...")
			{
				if (Branch->MaxIncludeRuleNumber > BestRuleNumber)
				{
					BestRuleNumber = Branch->MaxIncludeRuleNumber;
				}
			}
			else
			{
				if (Branch->IsMatch(Tokens[TokenIdx]))
				{
					BestRuleNumber = HighestPossibleIncludeMatch(Branch, Tokens, TokenIdx + 1, BestRuleNumber);
				}
			}
		}
	}
	return BestRuleNumber;
}

} // namespace UGSCore
