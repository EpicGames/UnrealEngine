// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UGSCore
{

struct FFileFilterNode;

/// <summary>
/// Indicates whether files which match a pattern should be included or excluded
/// </summary>
enum class EFileFilterType
{
	Include,
	Exclude,
};

/// <summary>
/// Stores a set of rules, similar to p4 path specifications, which can be used to efficiently filter files in or out of a set.
/// The rules are merged into a tree with nodes for each path fragments and rule numbers in each leaf, allowing tree traversal in an arbitrary order 
/// to how they are applied.
/// </summary>
class FFileFilter
{
public:
	/// <summary>
	/// Default constructor
	/// </summary>
	FFileFilter(EFileFilterType DefaultType = EFileFilterType::Exclude);

	/// <summary>
	/// Copy constructor
	/// </summary>
	/// <param name="Other">Filter to copy from</param>
	FFileFilter(const FFileFilter& Other);

	/// <summary>
	/// Constructs a file filter from a p4-style filespec. Exclude lines are prefixed with a - character.
	/// </summary>
	/// <param name="Lines">Lines to parse rules from</param>
	FFileFilter(const TArray<FString>& Lines);

	/// <summary>
	/// Destructor
	/// </summary>
	~FFileFilter();

	/// <summary>
	/// Reads a configuration file split into sections
	/// </summary>
	/// <param name="Filter"></param>
	/// <param name="RulesFileName"></param>
	/// <param name="Conditions"></param>
	void ReadRulesFromFile(const FString& FileName, const FString& SectionName);

	/// <summary>
	/// Adds an include or exclude rule to the filter, treating anything with a '-' prefix as an exclude rule, and anything else as an include rule.
	/// </summary>
	/// <param name="Pattern">The pattern which the rule should match</param>
	/// <param name="bInclude">Whether to include or exclude files matching this rule</param>
	void AddRule(const FString& Pattern);

	/// <summary>
	/// Adds an include or exclude rule to the filter
	/// </summary>
	/// <param name="Pattern">The pattern which the rule should match</param>
	/// <param name="bInclude">Whether to include or exclude files matching this rule</param>
	void AddRule(const FString& Pattern, EFileFilterType Type);

	/// <summary>
	/// Determines whether the given file matches the filter
	/// </summary>
	/// <param name="FileName">File to match</param>
	/// <returns>True if the file passes the filter</returns>
	bool Matches(const FString& FileName) const;

	/// <summary>
	/// Determines whether it's possible for anything within the given folder name to match the filter. Useful to early out of recursive file searches.
	/// </summary>
	/// <param name="FolderName">File to match</param>
	/// <returns>True if the file passes the filter</returns>
	bool PossiblyMatches(const FString& FolderName) const;

private:
	struct FNode;

	/// <summary>
	/// Root node for the tree.
	/// </summary>
	FNode* RootNode;

	/// <summary>
	/// The default node, which will match any path
	/// </summary>
	FNode* DefaultNode;

	/// <summary>
	/// Terminating nodes for each rule added to the filter
	/// </summary>
	TArray<FNode*> Rules;

	/// <summary>
	/// Finds the node which matches a given list of tokens.
	/// </summary>
	/// <param name="CurrentNode"></param>
	/// <param name="Tokens"></param>
	/// <param name="TokenIdx"></param>
	/// <param name="CurrentBestNode"></param>
	/// <returns></returns>
	static const FNode* FindMatchingNode(const FNode* CurrentNode, const TArray<FString>& Tokens, int TokenIdx, const FNode* CurrentBestNode);

	/// <summary>
	/// Returns the highest possible rule number which can match the given list of input tokens, assuming that the list of input tokens is incomplete.
	/// </summary>
	/// <param name="CurrentNode">The current node being checked</param>
	/// <param name="Tokens">The tokens to match</param>
	/// <param name="TokenIdx">Current token index</param>
	/// <param name="CurrentBestRuleNumber">The highest rule number seen so far. Used to optimize tree traversals.</param>
	/// <returns>New highest rule number</returns>
	static int HighestPossibleIncludeMatch(const FNode* CurrentNode, const TArray<FString>& Tokens, int TokenIdx, int CurrentBestRuleNumber);
};

} // namespace UGSCore
