// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindStreamsTask.h"

#define LOCTEXT_NAMESPACE "UnrealGameSync"

namespace UGSCore
{

	FFindStreamsTask::FFindStreamsTask(
		TSharedRef<FPerforceConnection> InPerforce,
		TSharedRef<FLineBasedTextWriter> InLog,
		TArray<FString>& OutStreams,
		const FString& StreamFilter)
		: Perforce(MoveTemp(InPerforce))
		, Log(MoveTemp(InLog))
		, Streams(OutStreams)
		, Filter(StreamFilter)
	{}

	FFindStreamsTask::~FFindStreamsTask()
	{}

	TSharedRef<FModalTaskResult> FFindStreamsTask::Run(FEvent* AbortEvent)
	{
		// TODO enable these if we want
		//try
		{
			return RunInternal(AbortEvent);
		}
		//catch(FAbortException)
		{
			//return FModalTaskResult::Aborted();
		}
	}

	TSharedRef<FModalTaskResult> FFindStreamsTask::RunInternal(FEvent* AbortEvent)
	{
		return Perforce->FindStreams(Filter, Streams, AbortEvent, *Log)
			? FModalTaskResult::Success()
			: FModalTaskResult::Failure(LOCTEXT("PerforceTaskFailure", "Failed to find streams"));
	}

#undef LOCTEXT_NAMESPACE

} // namespace UGSCore
