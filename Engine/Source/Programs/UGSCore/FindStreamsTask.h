// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ModalTask.h"
#include "Perforce.h"
#include "OutputAdapters.h"

namespace UGSCore
{

class FFindStreamsTask : public IModalTask
{
public:
	TSharedRef<FPerforceConnection> Perforce;

	FFindStreamsTask(
		TSharedRef<FPerforceConnection> InPerforce,
		TSharedRef<FLineBasedTextWriter> InLog,
		TArray<FString>& OutStreams,
		const FString& StreamFilter);
	virtual ~FFindStreamsTask() override;

	virtual TSharedRef<FModalTaskResult> Run(FEvent* AbortEvent) override;

private:
	TSharedRef<FLineBasedTextWriter> Log;

	TArray<FString>& Streams;
	FString Filter;

	TSharedRef<FModalTaskResult> RunInternal(FEvent* AbortEvent);
};

} // namespace UGSCore
