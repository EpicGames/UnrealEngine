// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModalTask.h"

namespace UGSCore
{

//// FModalTaskResult ////

FModalTaskResult::FModalTaskResult(bool bInSucceeded, const FText& InMessage)
	: bSucceeded(bInSucceeded)
	, Message(InMessage)
{
}

FModalTaskResult::~FModalTaskResult()
{
}

bool FModalTaskResult::Succeeded() const
{
	return bSucceeded;
}

bool FModalTaskResult::Failed() const
{
	return !bSucceeded;
}

const FText& FModalTaskResult::GetMessage() const
{
	return Message;
}

TSharedRef<FModalTaskResult> FModalTaskResult::Success()
{
	return MakeShared<FModalTaskResult>(true, FText::GetEmpty());
}

TSharedRef<FModalTaskResult> FModalTaskResult::Failure(const FText& Message)
{
	return MakeShared<FModalTaskResult>(false, Message);
}

TSharedRef<FModalTaskResult> FModalTaskResult::Aborted()
{
	return MakeShared<FModalTaskResult>(false, FText::GetEmpty());
}

//// IModalTask ////

IModalTask::~IModalTask()
{
}

} // namespace UGSCore
