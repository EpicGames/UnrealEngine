// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

namespace UGSCore
{

class FModalTaskResult
{
public:
	FModalTaskResult(bool bSucceeded, const FText& Message);
	~FModalTaskResult();

	bool Succeeded() const;
	bool Failed() const;
	const FText& GetMessage() const;

	static TSharedRef<FModalTaskResult> Success();
	static TSharedRef<FModalTaskResult> Failure(const FText& Message);
	static TSharedRef<FModalTaskResult> Aborted();

private:
	const bool bSucceeded;
	const FText Message;
};

class IModalTask
{
public:
	virtual ~IModalTask();
	virtual TSharedRef<FModalTaskResult> Run(FEvent* AbortEvent) = 0;
};

} // namespace UGSCore
