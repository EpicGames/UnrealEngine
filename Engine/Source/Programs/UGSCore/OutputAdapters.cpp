// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputAdapters.h"
#include "Utility.h"

namespace UGSCore
{

//// FLineBasedTextWriter ////

FLineBasedTextWriter::FLineBasedTextWriter()
{
}

FLineBasedTextWriter::~FLineBasedTextWriter()
{
}

void FLineBasedTextWriter::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	WriteLine(V);
}

void FLineBasedTextWriter::Write(TCHAR Character)
{
	if(Character == '\n')
	{
		FlushLine(CurrentLine);
		CurrentLine.Empty();
	}
	else if(Character != '\r')
	{
		CurrentLine.AppendChar(Character);
	}
}

void FLineBasedTextWriter::WriteLine(const FString& Line)
{
	TArray<FString> SubLines;
	Line.ParseIntoArray(SubLines, TEXT("\n"), false);

	for(FString& SubLine: SubLines)
	{
		FlushLine(SubLine.TrimEnd());
	}
}

//// FNullTextWriter ////

void FNullTextWriter::FlushLine(const FString& Line)
{
}

//// FBufferedTextWriter ////

FBufferedTextWriter::FBufferedTextWriter()
{
}

FBufferedTextWriter::~FBufferedTextWriter()
{
}

void FBufferedTextWriter::Attach(const TSharedRef<FLineBasedTextWriter>& InInner)
{
	Detach();

	Inner = InInner;

	for(const FString& BufferedLine : BufferedLines)
	{
		Inner->FlushLine(BufferedLine);
	}
	BufferedLines.Empty();
}

void FBufferedTextWriter::Detach()
{
	Inner.Reset();
}

void FBufferedTextWriter::FlushLine(const FString& Line)
{
	if(Inner.IsValid())
	{
		Inner->FlushLine(Line);
	}
	else
	{
		BufferedLines.Add(Line);
	}
}

//// FPrefixedTextWriter ////

FPrefixedTextWriter::FPrefixedTextWriter(const TCHAR* InPrefix, TSharedRef<FLineBasedTextWriter> InInner)
	: Prefix(InPrefix)
	, Inner(InInner)
{
}

FPrefixedTextWriter::~FPrefixedTextWriter()
{
}

void FPrefixedTextWriter::FlushLine(const FString& Line)
{
	Inner->WriteLine(Prefix + Line);
}

//// FBoundedLogWriter ////

FBoundedLogWriter::FBoundedLogWriter(const TCHAR* InFileName, int32 InMaxSize)
	: FileName(InFileName)
	, BackupFileName(FileName + TEXT(".bak"))
	, MaxSize(InMaxSize)
	, Inner(nullptr)
{
	OpenInner();
}

FBoundedLogWriter::~FBoundedLogWriter()
{
	CloseInner();
}

void FBoundedLogWriter::FlushLine(const FString& Line)
{
	/*
			if(Inner != null)
			{
				Inner.WriteLine(Line);
				if(Inner.BaseStream.Position > MaxSize)
				{
					CloseInner();
					OpenInner();
				}
			}
	*/
}

void FBoundedLogWriter::OpenInner()
{
/*	CloseInner();
	try
	{
		if(File.Exists(BackupFileName))
		{
			File.Delete(BackupFileName);
		}
		if(File.Exists(FileName))
		{
			File.Move(FileName, BackupFileName);
		}

		Inner = new StreamWriter(FileName);
		Inner.AutoFlush = true;
	}
	catch(Exception)
	{
		Inner = null;
	}*/
}

void FBoundedLogWriter::CloseInner()
{
	/*
	if(Inner != null)
	{
		Inner.Close();
		Inner = null;
	}*/
}

//// FComposedTextWriter ////

FComposedTextWriter::FComposedTextWriter(const TArray<TSharedRef<FLineBasedTextWriter>>& InInners)
	: Inners(InInners)
{
}

FComposedTextWriter::~FComposedTextWriter()
{
}

void FComposedTextWriter::FlushLine(const FString& Line)
{
	for(TSharedRef<FLineBasedTextWriter>& Inner : Inners)
	{
		Inner->FlushLine(Line);
	}
}

//// FProgressValue ////

FProgressValue::FProgressValue()
{
	Clear();
}

FProgressValue::~FProgressValue()
{
}

void FProgressValue::Clear()
{
	State = TTuple<FString,float>(TEXT("Starting..."), 0.0f);

	Ranges.Empty();
	Ranges.Add(TTuple<float, float>(0.0f, 1.0f));
}

TTuple<FString, float> FProgressValue::GetCurrent() const
{
	return State;
}

void FProgressValue::Set(const TCHAR* Message)
{
	if(Ranges.Num() == 1)
	{
		State = TTuple<FString,float>(Message, State.Value);
	}
}

void FProgressValue::Set(const TCHAR* Message, float Fraction)
{
	if(Ranges.Num() == 1)
	{
		State = TTuple<FString, float>(Message, RelativeToAbsoluteFraction(Fraction));
	}
	else
	{
		State = TTuple<FString, float>(State.Key, RelativeToAbsoluteFraction(Fraction));
	}
}

void FProgressValue::Set(float Fraction)
{
	State = TTuple<FString, float>(State.Key, RelativeToAbsoluteFraction(Fraction));
}

void FProgressValue::Increment(float Fraction)
{
	Set(State.Value + RelativeToAbsoluteFraction(Fraction));
}

void FProgressValue::Push(float MaxFraction)
{
	Ranges.Push(TTuple<float,float>(State.Value, RelativeToAbsoluteFraction(MaxFraction)));
}

void FProgressValue::Pop()
{
	if(Ranges.Num() > 1)
	{
		State = TTuple<FString,float>(State.Key, Ranges.Pop().Value);
	}
}

float FProgressValue::RelativeToAbsoluteFraction(float Fraction)
{
	TTuple<float, float> Range = Ranges.Top();
	return Range.Key + (Range.Value - Range.Key) * Fraction;
}

//// FProgressTextWriter ////

const FString FProgressTextWriter::DirectivePrefix = TEXT("@progress ");

FProgressTextWriter::FProgressTextWriter(const FProgressValue& InValue, const TSharedRef<FLineBasedTextWriter>& InInner)
	: Value(InValue)
	, Inner(InInner)
{ 
}

FProgressTextWriter::~FProgressTextWriter()
{
}

void FProgressTextWriter::FlushLine(const FString& Line)
{
	if(Line.StartsWith(DirectivePrefix))
	{
		// Line that just contains a progress directive
		bool bSkipLine = false;
		ProcessInternal(Line.Mid(DirectivePrefix.Len()), bSkipLine);
	}
	else
	{
		bool bSkipLine = false;
		FString RemainingLine = Line;

		// Look for a progress directive at the end of a line, in square brackets
		FString TrimLine = Line.TrimEnd();
		if(TrimLine.EndsWith(TEXT("]")))
		{
			for(int LastIdx = TrimLine.Len() - 2; LastIdx >= 0 && TrimLine[LastIdx] != ']'; LastIdx--)
			{
				if(TrimLine[LastIdx] == '[')
				{
					FString DirectiveSubstring = TrimLine.Mid(LastIdx + 1, TrimLine.Len() - LastIdx - 2);
					if(DirectiveSubstring.StartsWith(DirectivePrefix))
					{
						ProcessInternal(DirectiveSubstring.Mid(DirectivePrefix.Len()), bSkipLine);
						RemainingLine = Line.Mid(0, LastIdx).TrimEnd();
					}
					break;
				}
			}
		}

		if(!bSkipLine)
		{
			Inner->WriteLine(RemainingLine);
		}
	}
}

void FProgressTextWriter::ProcessInternal(const FString& Line, bool& bSkipLine)
{
	TArray<FString> Tokens = ParseTokens(Line);
	for(int TokenIdx = 0; TokenIdx < Tokens.Num(); )
	{
		float Fraction;
		if(ReadFraction(Tokens, TokenIdx, Fraction))
		{
			Value.Set(Fraction);
		}
		else if(Tokens[TokenIdx] == TEXT("push"))
		{
			TokenIdx++;
			if(ReadFraction(Tokens, TokenIdx, Fraction))
			{
				Value.Push(Fraction);
			}
		}
		else if(Tokens[TokenIdx] == TEXT("pop"))
		{
			TokenIdx++;
			Value.Pop();
		}
		else if(Tokens[TokenIdx] == TEXT("increment"))
		{
			TokenIdx++;
			if(ReadFraction(Tokens, TokenIdx, Fraction))
			{
				Value.Increment(Fraction);
			}
		}
		else if(Tokens[TokenIdx] == TEXT("skipline"))
		{
			TokenIdx++;
			bSkipLine = true;
		}
		else if(Tokens[TokenIdx].Len() >= 2 && ((Tokens[TokenIdx].StartsWith(TEXT("\'")) && Tokens[TokenIdx].EndsWith(TEXT("\'"))) || (Tokens[TokenIdx].StartsWith(TEXT("\"")) && Tokens[TokenIdx].EndsWith(TEXT("\"")))))
		{
			const FString& Message = Tokens[TokenIdx++];
			Value.Set(*Message.Mid(1, Message.Len() - 2));
		}
		else
		{
			TokenIdx++;
		}
	}
}

TArray<FString> FProgressTextWriter::ParseTokens(const FString& Line)
{
	TArray<FString> Tokens;
	for(int Idx = 0;;)
	{
		// Skip whitespace
		while(Idx < Line.Len() && FChar::IsWhitespace(Line[Idx]))
		{
			Idx++;
		}
		if(Idx == Line.Len())
		{
			break;
		}

		// Read the next token
		if(FChar::IsAlnum(Line[Idx]))
		{
			int StartIdx = Idx++;
			while(Idx < Line.Len() && FChar::IsAlnum(Line[Idx]))
			{
				Idx++;
			}
			Tokens.Add(Line.Mid(StartIdx, Idx - StartIdx));
		}
		else if(Line[Idx] == '\'' || Line[Idx] == '\"')
		{
			int StartIdx = Idx++;
			while(Idx < Line.Len() && Line[Idx] != Line[StartIdx])
			{
				Idx++;
			}
			Tokens.Add(Line.Mid(StartIdx, ++Idx - StartIdx));
		}
		else
		{
			Tokens.Add(Line.Mid(Idx++, 1));
		}
	}
	return Tokens;
}

bool FProgressTextWriter::ReadFraction(const TArray<FString>& Tokens, int& TokenIdx, float& Fraction)
{
	// Read a fraction in the form x%
	if(TokenIdx + 2 <= Tokens.Num() && Tokens[TokenIdx + 1] == TEXT("%"))
	{
		int Numerator;
		if(FUtility::TryParse(*Tokens[TokenIdx], Numerator))
		{
			Fraction = (float)Numerator / 100.0f;
			TokenIdx += 2;
			return true;
		}
	}
			
	// Read a fraction in the form x/y
	if(TokenIdx + 3 <= Tokens.Num() && Tokens[TokenIdx + 1] == "/")
	{
		int Numerator, Denominator;
		if(FUtility::TryParse(*Tokens[TokenIdx], Numerator) && FUtility::TryParse(*Tokens[TokenIdx + 2], Denominator))
		{
			Fraction = (float)Numerator / (float)Denominator;
			TokenIdx += 3;
			return true;
		}
	}

	Fraction = 0.0f;
	return false;
}

} // namespace UGSCore
