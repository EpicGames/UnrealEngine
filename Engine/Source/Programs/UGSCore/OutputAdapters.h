// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UGSCore
{

class FLineBasedTextWriter : public FOutputDevice
{
public:
	FLineBasedTextWriter();
	virtual ~FLineBasedTextWriter();
	void Write(TCHAR Character);
	void WriteLine(const FString& Line);
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override final;
	virtual void FlushLine(const FString& Line) = 0;

private:
	FString CurrentLine;
};

class FNullTextWriter : public FLineBasedTextWriter
{
public:
	virtual void FlushLine(const FString& Line) override;
};

class FBufferedTextWriter : public FLineBasedTextWriter
{
public:
	FBufferedTextWriter();
	virtual ~FBufferedTextWriter();

	void Attach(const TSharedRef<FLineBasedTextWriter>& InInner);
	void Detach();

	virtual void FlushLine(const FString& Line) override;

private:
	TArray<FString> BufferedLines;
	TSharedPtr<FLineBasedTextWriter> Inner;
};

class FPrefixedTextWriter : public FLineBasedTextWriter
{
public:
	FPrefixedTextWriter(const TCHAR* InPrefix, TSharedRef<FLineBasedTextWriter> InInner);
	virtual ~FPrefixedTextWriter() override;
	virtual void FlushLine(const FString& Line) override;

private:
	FString Prefix;
	TSharedRef<FLineBasedTextWriter> Inner;
};

class FBoundedLogWriter : public FLineBasedTextWriter
{
public:
	FBoundedLogWriter(const TCHAR* InFileName, int32 InMaxSize);
	virtual ~FBoundedLogWriter() override;
	virtual void FlushLine(const FString& Line) override;

private:
	FString FileName;
	FString BackupFileName;
	int32 MaxSize;
	FArchive* Inner;

	void OpenInner();
	void CloseInner();
};

class FComposedTextWriter : public FLineBasedTextWriter
{
public:
	FComposedTextWriter(const TArray<TSharedRef<FLineBasedTextWriter>>& InInners);
	virtual ~FComposedTextWriter() override;
	virtual void FlushLine(const FString& Line) override;

private:
	TArray<TSharedRef<FLineBasedTextWriter>> Inners;
};

class FProgressValue
{
public:
	FProgressValue();
	~FProgressValue();

	void Clear();
	TTuple<FString, float> GetCurrent() const;
	void Set(const TCHAR* Message);
	void Set(const TCHAR* Message, float Fraction);
	void Set(float Fraction);
	void Increment(float Fraction);
	void Push(float MaxFraction);
	void Pop();

private:
	TTuple<FString, float> State;
	TArray<TTuple<float, float>> Ranges;

	float RelativeToAbsoluteFraction(float Fraction);
};

class FProgressTextWriter : public FLineBasedTextWriter
{
public:
	FProgressTextWriter(const FProgressValue& InValue, const TSharedRef<FLineBasedTextWriter>& InInner);
	virtual ~FProgressTextWriter() override;

	virtual void FlushLine(const FString& Line) override;

private:
	static const FString DirectivePrefix;

	FProgressValue Value;
	TSharedRef<FLineBasedTextWriter> Inner;

	void ProcessInternal(const FString& Line, bool& bSkipLine);
	static TArray<FString> ParseTokens(const FString& Line);
	static bool ReadFraction(const TArray<FString>& Tokens, int& TokenIdx, float& Fraction);
};

} // namespace UGSCore
