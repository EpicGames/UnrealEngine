// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perforce.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Event.h"
#include "Utility.h"

namespace UGSCore
{

bool TryGetValue(const TMap<FString, FString>& Map, const TCHAR* Key, FString& OutValue)
{
	const FString* Value = Map.Find(Key);
	if(Value == nullptr)
	{
		OutValue = FString();
		return false;
	}
	else
	{
		OutValue = *Value;
		return true;
	}
}

TArray<FString> Split(const FString& Text, const TCHAR* Delim, bool bRemoveEmptyEntries = false)
{
	TArray<FString> Result;
	Text.ParseIntoArray(Result, Delim, bRemoveEmptyEntries);
	return Result;
}

FString GetTempFileName()
{
	return FPaths::EngineIntermediateDir() / FGuid::NewGuid().ToString();
}

struct FPerforceExe
{
	bool bValidP4 = false;

	// Default assumed locations, we will try to find the real path
#if PLATFORM_WINDOWS
	FString ExecutablePath = TEXT("C:\\Program Files (x86)\\Perforce\\p4.exe");
#else
	FString ExecutablePath = TEXT("/usr/bin/p4");
#endif

	FString GetPerforceExe()
	{
		static bool bExecutablePathCached = false;
		if (bExecutablePathCached)
		{
			return ExecutablePath;
		}

		bExecutablePathCached = true;

#if PLATFORM_MAC || PLATFORM_LINUX
		int32 ReturnCode = -1;
		FString OutResults;
		FString OutErrors;

#if PLATFORM_MAC
		FPlatformProcess::ExecProcess(TEXT("/usr/bin/mdfind"), TEXT("\"kMDItemFSName = 'p4' && kMDItemContentType = 'public.unix-executable'\""), &ReturnCode, &OutResults, &OutErrors);
		// in case it returns multiple p4's, pick the first one
		OutResults = Split(OutResults, LINE_TERMINATOR)[0];
#elif PLATFORM_LINUX
		FPlatformProcess::ExecProcess(TEXT("which"), TEXT("p4"), &ReturnCode, &OutResults, &OutErrors);
#endif // PLATFORM_MAC

		if (ReturnCode != 0)
		{
			// Todo: Log OutErrors somehow (should this take function the output log?)
			// for now just return our default paths
			return ExecutablePath;
		}

		ExecutablePath = OutResults.TrimEnd();
#endif // PLATFORM_MAC || PLATFORM_LINUX

		return ExecutablePath;
	}

	inline int InnerRunCommand(const FString& CommandLine, TArray<FString>& OutLines, FEvent* AbortEvent, const FString WriteChildText = FString())
	{
		void* ParentReadPipe  = nullptr;
		void* ParentWritePipe = nullptr;
		void* ChildReadPipe   = nullptr;
		void* ChildWritePipe  = nullptr;

		// Create pipe for reading from child stdout
		FPlatformProcess::CreatePipe(ParentReadPipe, ChildWritePipe);

		// Create pipe for writing to child stdin
		const bool bWriting = !WriteChildText.IsEmpty();
		if (bWriting)
		{
			FPlatformProcess::CreatePipe(ChildReadPipe, ParentWritePipe, true);
		}

		FProcHandle P4Proc = FPlatformProcess::CreateProc(*GetPerforceExe(), *CommandLine, false, true, true, nullptr, 0, nullptr, ChildWritePipe, ChildReadPipe);

		// Write to child process's stdin
		if (FPlatformProcess::IsProcRunning(P4Proc) && bWriting)
		{
			FPlatformProcess::WritePipe(ParentWritePipe, WriteChildText);
		}

		// Done writing
		FPlatformProcess::ClosePipe(ChildReadPipe, ParentWritePipe);

		// Read from child process's stdout
		FString P4Output;
		FString LatestOutput = FPlatformProcess::ReadPipe(ParentReadPipe);
		while (FPlatformProcess::IsProcRunning(P4Proc) || !LatestOutput.IsEmpty())
		{
			P4Output += LatestOutput;
			LatestOutput = FPlatformProcess::ReadPipe(ParentReadPipe);

			if (AbortEvent->Wait(FTimespan::Zero()))
			{
				FPlatformProcess::TerminateProc(P4Proc);
				FPlatformProcess::ClosePipe(ParentReadPipe, ChildWritePipe);
				FPlatformProcess::CloseProc(P4Proc);
				return -1;
			}
		}

		// Done reading
		FPlatformProcess::ClosePipe(ParentReadPipe, ChildWritePipe);

		OutLines = Split(P4Output, LINE_TERMINATOR);

		int ExitCode = -1;

		const bool bGotReturnCode = FPlatformProcess::GetProcReturnCode(P4Proc, &ExitCode);
		
		FPlatformProcess::CloseProc(P4Proc);
		
		if (bGotReturnCode)
		{
			return ExitCode;
		}

		return -1;
	}

	int RunCommand(const FString& CommandLine, TArray<FString>& OutLines, FEvent* AbortEvent, const FString& WritePipeText = FString())
	{
		if (bValidP4)
		{
			return InnerRunCommand(CommandLine, OutLines, AbortEvent, WritePipeText);
		}

		return -1;
	}

	void VerifyPerforcePath()
	{
#if PLATFORM_WINDOWS
		bValidP4 = FPaths::FileExists(ExecutablePath);
		if (!bValidP4)
		{
			ExecutablePath.ReplaceInline(TEXT(" (x86)"), TEXT(""));
			bValidP4 = FPaths::FileExists(ExecutablePath);
		}
		else
#endif
		{
			bValidP4 = true;
		}
	}

	FPerforceExe()
	{
		VerifyPerforcePath();
	}
};
static FPerforceExe GPerforceExe;

//// FPerforceChangeSummary ////

FPerforceChangeSummary::FPerforceChangeSummary()
	: Number(0)
	, Date(0)
{
}

//// FPerforceFileChangeSummary ////

FPerforceFileChangeSummary::FPerforceFileChangeSummary()
	: Revision(0)
	, ChangeNumber(0)
	, Date(0)
{
}

//// FPerforceClientRecord ////

FPerforceClientRecord::FPerforceClientRecord(const TMap<FString, FString>& Tags)
{
	TryGetValue(Tags, TEXT("client"), Name);
	TryGetValue(Tags, TEXT("Owner"), Owner);
	TryGetValue(Tags, TEXT("Host"), Host);
	TryGetValue(Tags, TEXT("Root"), Root);
}

//// FPerforceDescribeFileRecord ////

FPerforceDescribeFileRecord::FPerforceDescribeFileRecord()
	: Revision(0)
	, FileSize(0)
{
}

//// FPerforceDescribeRecord ////

FPerforceDescribeRecord::FPerforceDescribeRecord(const TMap<FString, FString>& Tags)
{
	FString ChangeString;
	if(!TryGetValue(Tags, TEXT("change"), ChangeString) || !FUtility::TryParse(*ChangeString, ChangeNumber))
	{
		ChangeNumber = -1;
	}

	TryGetValue(Tags, TEXT("user"), User);
	TryGetValue(Tags, TEXT("client"), Client);

	FString TimeString;
	if(!TryGetValue(Tags, TEXT("time"), TimeString) || !FUtility::TryParse(*TimeString, Time))
	{
		Time = -1;
	}

	TryGetValue(Tags, TEXT("desc"), Description);
	TryGetValue(Tags, TEXT("status"), Status);
	TryGetValue(Tags, TEXT("changeType"), ChangeType);
	TryGetValue(Tags, TEXT("path"), Path);

	for(int Idx = 0;;Idx++)
	{
		FString Suffix = FString::Printf(TEXT("%d"), Idx);

		FPerforceDescribeFileRecord File;
		if(!TryGetValue(Tags, *(FString(TEXT("depotFile")) + Suffix), File.DepotFile))
		{
			break;
		}

		TryGetValue(Tags, *(FString(TEXT("action")) + Suffix), File.Action);
		TryGetValue(Tags, *(FString(TEXT("type")) + Suffix), File.Type);

		FString RevisionString;
		if(!TryGetValue(Tags, *(FString(TEXT("rev")) + Suffix), RevisionString) || !FUtility::TryParse(*RevisionString, File.Revision))
		{
			File.Revision = -1;
		}

		FString FileSizeString;
		if(!TryGetValue(Tags, *(FString(TEXT("fileSize")) + Suffix), FileSizeString) || !FUtility::TryParse(*FileSizeString, File.FileSize))
		{
			File.FileSize = -1;
		}

		TryGetValue(Tags, *(FString(TEXT("digest")) + Suffix), File.Digest);
		Files.Add(File);
	}
}

//// FPerforceInfoRecord ////

FPerforceInfoRecord::FPerforceInfoRecord(const TMap<FString, FString>& Tags)
	: ServerTimeZone(0, 0, 0)
{
	TryGetValue(Tags, TEXT("userName"), UserName);
	TryGetValue(Tags, TEXT("clientHost"), HostName);
	TryGetValue(Tags, TEXT("clientAddress"), ClientAddress);
	TryGetValue(Tags, TEXT("serverAddress"), ServerAddress);

	FString ServerDateTime;
	if(TryGetValue(Tags, TEXT("serverDate"), ServerDateTime))
	{
		TArray<FString> Fields;
		ServerDateTime.ParseIntoArray(Fields, TEXT(" "));
		if(Fields.Num() >= 3)
		{
			int32 Offset;
			if(FUtility::TryParse(*Fields[2], Offset))
			{
				ServerTimeZone = (Offset < 0)? -FTimespan(-Offset / 100, -Offset % 100, 0) : FTimespan(Offset / 100, Offset % 100, 0);
			}
		}
	}
}

//// FPerforceFileRecord ////

FPerforceFileRecord::FPerforceFileRecord(const TMap<FString, FString>& Tags)
	: Revision(0)
{
	TryGetValue(Tags, TEXT("depotFile"), DepotPath);
	TryGetValue(Tags, TEXT("clientFile"), ClientPath);
	TryGetValue(Tags, TEXT("path"), Path);
	if(!TryGetValue(Tags, TEXT("action"), Action))
	{
		TryGetValue(Tags, TEXT("headAction"), Action);
	}
	IsMapped = Tags.Contains(TEXT("isMapped"));
	Unmap = Tags.Contains(TEXT("unmap"));

	FString RevisionString;
	if(TryGetValue(Tags, TEXT("rev"), RevisionString))
	{
		FUtility::TryParse(*RevisionString, Revision);
	}
}

//// FPerforceTagRecordParser ////

FPerforceTagRecordParser::FPerforceTagRecordParser(TFunction<void (const TMap<FString, FString>&)> InOutputRecord)
	: OutputRecord(InOutputRecord)
{
}

FPerforceTagRecordParser::~FPerforceTagRecordParser()
{
	Flush();
}

void FPerforceTagRecordParser::OutputLine(const FString& Line)
{
	int SpaceIdx;
	if(!Line.FindChar(TEXT(' '), SpaceIdx))
	{
		SpaceIdx = INDEX_NONE;
	}

	FString Key = (SpaceIdx > 0)? Line.Left(SpaceIdx) : Line;
	if(Tags.Contains(Key))
	{
		OutputRecord(Tags);
		Tags.Empty();
	}

	Tags.FindOrAdd(Key) = (SpaceIdx > 0)? Line.Mid(SpaceIdx + 1) : TEXT("");
}

void FPerforceTagRecordParser::Flush()
{
	if(Tags.Num() > 0)
	{
		OutputRecord(Tags);
		Tags.Empty();
	}
}

//// FPerforceSyncOptions ////

FPerforceSyncOptions::FPerforceSyncOptions()
	: NumRetries(0)
	, NumThreads(0)
	, TcpBufferSize(0)
{
}

//// FPerforceSpec ////

const TCHAR* FPerforceSpec::GetField(const TCHAR* Name) const
{
	for(const TTuple<FString, FString>& Section : Sections)
	{
		if(Section.Key == Name)
		{
			return *Section.Value;
		}
	}
	return nullptr;
}

void FPerforceSpec::SetField(const TCHAR* Name, const TCHAR* Value)
{
	for(TTuple<FString, FString>& Section : Sections)
	{
		if(Section.Key == Name)
		{
			Section.Value = Value;
			return;
		}
	}
	Sections.Add(TTuple<FString, FString>(Name, Value));
}

bool FPerforceSpec::TryParse(const TArray<FString>& Lines, TSharedPtr<FPerforceSpec>& OutSpec, FOutputDevice& Log)
{
	TSharedPtr<FPerforceSpec> Spec = MakeShared<FPerforceSpec>();
	for(int LineIdx = 0; LineIdx < Lines.Num(); LineIdx++)
	{
		FString Line = Lines[LineIdx].TrimStartAndEnd();
		if(Line.Len() > 0 && Lines[LineIdx][0] != '#')
		{
			// Read the section name
			int SeparatorIdx;
			if(!Lines[LineIdx].FindChar(':', SeparatorIdx))
			{
				SeparatorIdx = -1;
			}
			if(SeparatorIdx == -1 || !FChar::IsAlpha(Lines[LineIdx][0]))
			{
				Log.Logf(TEXT("Invalid spec format at line %d: \"%s\""), LineIdx, *Lines[LineIdx]);
				return false;
			}

			// Get the section name
			FString SectionName = Lines[LineIdx].Mid(0, SeparatorIdx);

			// Parse the section value
			FString Value = Lines[LineIdx].Mid(SeparatorIdx + 1).TrimStartAndEnd();
			for(; LineIdx + 1 < Lines.Num(); LineIdx++)
			{
				if(Lines[LineIdx + 1].Len() == 0)
				{
					Value += TEXT("\n");
				}
				else if(Lines[LineIdx + 1][0] == '\t')
				{
					Value += Lines[LineIdx + 1].Mid(1) + TEXT("\n");
				}
				else
				{
					break;
				}
			}
			Value.TrimEndInline();

			Spec->Sections.Add(TTuple<FString,FString>(SectionName, Value));
		}
	}
	OutSpec = Spec;
	return true;
}

FString FPerforceSpec::ToString() const 
{
	FString Result;
	for(const TTuple<FString, FString>& Section : Sections)
	{
		if(Section.Value.Contains(TEXT("\n")))
		{
			Result += Section.Key + TEXT(":\n\t") + Section.Value.Replace(TEXT("\n"), TEXT("\n\t")) + LINE_TERMINATOR;
		}
		else
		{
			Result += Section.Key + TEXT(":\t") + Section.Value + LINE_TERMINATOR;
		}
		Result += LINE_TERMINATOR;
	}
	return Result;
}

//// FPerforceOutputLine ////

FPerforceOutputLine::FPerforceOutputLine()
	: Channel(EPerforceOutputChannel::Unknown)
{
}

FPerforceOutputLine::FPerforceOutputLine(EPerforceOutputChannel InChannel, const FString& InText)
	: Channel(InChannel)
	, Text(InText)
{
}

//// FPerforceConnection ////

FPerforceConnection::FPerforceConnection(const TCHAR* InUserName, const TCHAR* InClientName, const TCHAR* InServerAndPort)
	: ServerAndPort((InServerAndPort != nullptr)? InServerAndPort : TEXT(""))
	, UserName((InUserName != nullptr)? InUserName : TEXT(""))
	, ClientName((InClientName != nullptr)? InClientName : TEXT(""))
{
}

TSharedRef<FPerforceConnection> FPerforceConnection::OpenClient(const FString& NewClientName) const
{
	return MakeShared<FPerforceConnection>(*UserName, *NewClientName, *ServerAndPort);
}

bool FPerforceConnection::Info(TSharedPtr<FPerforceInfoRecord>& OutInfo, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<TMap<FString, FString>> TagRecords;
	if(!RunCommand(TEXT("info -s"), TagRecords, ECommandOptions::NoClient, AbortEvent, Log) || TagRecords.Num() != 1)
	{
		OutInfo = TSharedPtr<FPerforceInfoRecord>();
		return false;
	}
	else
	{
		OutInfo = MakeShared<FPerforceInfoRecord>(TagRecords[0]);
		return true;
	}
}

bool FPerforceConnection::GetSetting(const FString& Name, FString& Value, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<FString> Lines;
	if(!RunCommand(FString::Printf(TEXT("set %s"), *Name), Lines, ECommandOptions::NoChannels, AbortEvent, Log) || Lines.Num() != 1)
	{
		Value = FString();
		return false;
	}
	if(Lines[0].Len() <= Name.Len() || !Lines[0].StartsWith(Name) || Lines[0][Name.Len()] != '=')
	{
		Value = FString();
		return false;
	}

	Value = Lines[0].Mid(Name.Len() + 1);

	int EndIdx = Value.Find(TEXT(" ("));
	if(EndIdx != INDEX_NONE)
	{
		Value = Value.Mid(0, EndIdx).TrimStartAndEnd();
	}
	return true;
}

bool FPerforceConnection::FindClients(TArray<FPerforceClientRecord>& Clients, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommand(TEXT("clients"), Clients, ECommandOptions::NoClient, AbortEvent, Log);
}

bool FPerforceConnection::FindClients(TArray<FPerforceClientRecord>& Clients, const FString& ForUserName, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<FString> Lines;
	if(!RunCommand(FString::Printf(TEXT("clients -u%s"), *ForUserName), Clients, ECommandOptions::NoClient, AbortEvent, Log))
	{
		return false;
	}

	for(const FString& Line : Lines)
	{
		TArray<FString> Tokens = Split(Line, TEXT(" "), true);
		if(Tokens.Num() < 5 || Tokens[0] != TEXT("Client") || Tokens[3] != TEXT("root"))
		{
			Log.Logf(TEXT("Couldn't parse client from line '%s'"), *Line);
		}
	}
	return true;
}

bool FPerforceConnection::TryGetClientSpec(const FString& InClientName, TSharedPtr<FPerforceSpec>& OutSpec, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<FString> Lines;
	if(!RunCommand(FString::Printf(TEXT("client -o %s"), *InClientName), Lines, ECommandOptions::None, AbortEvent, Log))
	{
		OutSpec = nullptr;
		return false;
	}
	if(!FPerforceSpec::TryParse(Lines, OutSpec, Log))
	{
		OutSpec = nullptr;
		return false;
	}
	return true;
}

bool FPerforceConnection::CreateClient(const FPerforceClientRecord& Client, const FString& Stream, FEvent* AbortEvent, FOutputDevice& Log) const
{
	const FString PipeInput = FString::Printf(
		TEXT("Client: %s\nStream: %s\nRoot: %s\nOwner: %s\nHost: %s\nDescription: Created by %s in SlateUGS\n"),
		*Client.Name, *Stream, *Client.Root, *Client.Owner, *Client.Host, *Client.Owner);

	return RunCommand(TEXT("client -i"), ECommandOptions::None, AbortEvent, Log, PipeInput);
}

bool FPerforceConnection::TryGetStreamSpec(const FString& StreamName, TSharedPtr<FPerforceSpec>& OutSpec, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<FString> Lines;
	if(!RunCommand(FString::Printf(TEXT("stream -o %s"), *StreamName), Lines, ECommandOptions::None, AbortEvent, Log))
	{
		OutSpec = nullptr;
		return false;
	}
	if(!FPerforceSpec::TryParse(Lines, OutSpec, Log))
	{
		OutSpec = nullptr;
		return false;
	}
	return true;
}

bool FPerforceConnection::FindFiles(const FString& Filter, TArray<FPerforceFileRecord>& OutFileRecords, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommand(FString::Printf(TEXT("fstat \"%s\""), *Filter), OutFileRecords, ECommandOptions::None, AbortEvent, Log);
}

bool FPerforceConnection::Print(const FString& DepotPath, TArray<FString>& OutLines, FEvent* AbortEvent, FOutputDevice& Log) const
{
	FString TempFileName = GetTempFileName();
	if(!PrintToFile(DepotPath, TempFileName, AbortEvent, Log))
	{
		return false;
	}
	else
	{
		return FFileHelper::LoadFileToStringArray(OutLines, *TempFileName);
	}
}

bool FPerforceConnection::PrintToFile(const FString& DepotPath, const FString& OutputFileName, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommand(FString::Printf(TEXT("print -q -o \"%s\" \"%s\""), *OutputFileName, *DepotPath), ECommandOptions::None, AbortEvent, Log);
}

bool FPerforceConnection::FileExists(const FString& Filter, bool& bExists, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<FPerforceFileRecord> FileRecords;
	if(RunCommand(FString::Printf(TEXT("fstat \"%s\""), *Filter), FileRecords, ECommandOptions::IgnoreNoSuchFilesError | ECommandOptions::IgnoreFilesNotInClientViewError, AbortEvent, Log))
	{
		bExists = (FileRecords.Num() > 0);
		return true;
	}
	else
	{
		bExists = false;
		return false;
	}
}

bool FPerforceConnection::FindChanges(const FString& Filter, int MaxResults, TArray<FPerforceChangeSummary>& OutChanges, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<FString> Filters;
	Filters.Add(Filter);
	return FindChanges(Filters, MaxResults, OutChanges, AbortEvent, Log);
}

bool FPerforceConnection::FindChanges(const TArray<FString>& Filters, int MaxResults, TArray<FPerforceChangeSummary>& OutChanges, FEvent* AbortEvent, FOutputDevice& Log) const
{
	OutChanges.Empty();

	FString Arguments = TEXT("changes -s submitted -t -L");
	if(MaxResults > 0)
	{
		Arguments += FString::Printf(TEXT(" -m %d"), MaxResults);
	}
	for(const FString& Filter : Filters)
	{
		Arguments += FString::Printf(TEXT(" \"%s\""), *Filter);
	}

	TArray<FString> Lines;
	if(!RunCommand(Arguments, Lines, ECommandOptions::None, AbortEvent, Log))
	{
		return false;
	}

	for(int Idx = 0; Idx < Lines.Num(); Idx++)
	{
		FPerforceChangeSummary Change;
		if(!TryParseChangeSummary(Lines[Idx], Change))
		{
			Log.Logf(TEXT("Couldn't parse description from '%s'"), *Lines[Idx]);
		}
		else
		{
			for(; Idx + 1 < Lines.Num(); Idx++)
			{
				if(Lines[Idx + 1].Len() == 0)
				{
					Change.Description += TEXT("\n");
				}
				else if(Lines[Idx + 1].StartsWith("\t"))
				{
					Change.Description += Lines[Idx + 1].Mid(1) + TEXT("\n");
				}
				else
				{
					break;
				}
			}
			Change.Description = Change.Description.TrimStartAndEnd();

			OutChanges.Add(Change);
		}
	}

	Algo::Sort(OutChanges, [](const FPerforceChangeSummary& A, const FPerforceChangeSummary& B){ return A.Number > B.Number; });

	for(int Idx = OutChanges.Num() - 1; Idx > 0; Idx--)
	{
		if(OutChanges[Idx - 1].Number == OutChanges[Idx].Number)
		{
			OutChanges.RemoveAt(Idx);
		}
	}

	if(MaxResults >= 0 && MaxResults < OutChanges.Num())
	{
		OutChanges.RemoveAt(MaxResults, OutChanges.Num() - MaxResults);
	}

	return true;
}

bool FPerforceConnection::TryParseChangeSummary(const FString& Line, FPerforceChangeSummary& Change) const
{
	TArray<FString> Tokens = Split(Line, TEXT(" "), true);
	if(Tokens.Num() == 7 && Tokens[0] == "Change" && Tokens[2] == "on" && Tokens[5] == "by")
	{
		if(FUtility::TryParse(*Tokens[1], Change.Number) && FPerforceUtils::TryParseDateTime(Tokens[3], Tokens[4], Change.Date))
		{
			int UserClientIdx;
			if(Tokens[6].FindChar(TEXT('@'), UserClientIdx))
			{
				Change.User = Tokens[6].Mid(0, UserClientIdx);
				Change.Client = Tokens[6].Mid(UserClientIdx + 1);
				return true;
			}
		}
	}
	return false;
}

bool FPerforceConnection::FindFileChanges(const FString& FilePath, int MaxResults, TArray<FPerforceFileChangeSummary>& Changes, FEvent* AbortEvent, FOutputDevice& Log) const
{
	FString Arguments = TEXT("filelog -L -t");
	if(MaxResults > 0)
	{
		Arguments += FString::Printf(TEXT(" -m %d"), MaxResults);
	}
	Arguments += FString::Printf(TEXT(" \"%s\""), *FilePath);

	TArray<FString> Lines;
	if(!RunCommand(Arguments, EPerforceOutputChannel::TaggedInfo, Lines, ECommandOptions::None, AbortEvent, Log))
	{
		return false;
	}

	for(int Idx = 0; Idx < Lines.Num(); Idx++)
	{
		FPerforceFileChangeSummary Change;
		if(!TryParseFileChangeSummary(Lines, Idx, Change))
		{
			Log.Logf(TEXT("Couldn't parse description from '%s'"), *Lines[Idx]);
		}
		else
		{
			Changes.Add(Change);
		}
	}

	return true;
}

bool FPerforceConnection::TryParseFileChangeSummary(const TArray<FString>& Lines, int& LineIdx, FPerforceFileChangeSummary& OutChange) const
{
	TArray<FString> Tokens = Split(Lines[LineIdx].TrimStartAndEnd(), TEXT(" "), true);
	if(Tokens.Num() != 10 || !Tokens[0].StartsWith(TEXT("#")) || Tokens[1] != TEXT("change") || Tokens[4] != TEXT("on") || Tokens[7] != TEXT("by"))
	{
		return false;
	}

	// Replace [5] date and [6] time with . as FDateTime expects format to be (yyyy.mm.dd-hh.mm.ss)
	Tokens[5].ReplaceCharInline('/', '.');
	Tokens[6].ReplaceCharInline(':', '.');
	if(!FUtility::TryParse(*Tokens[0].Mid(1), OutChange.Revision) || !FUtility::TryParse(*Tokens[2], OutChange.ChangeNumber) || !FDateTime::Parse(Tokens[5] + TEXT("-") + Tokens[6], OutChange.Date))
	{
		return false;
	}

	int UserClientIdx;
	if(!Tokens[8].FindChar('@', UserClientIdx))
	{
		return false;
	}

	OutChange.Action = Tokens[3];
	OutChange.Type = Tokens[9];
	if(OutChange.Type.StartsWith(TEXT("(")))
	{
		OutChange.Type = OutChange.Type.Mid(1);
	}
	if(OutChange.Type.EndsWith(TEXT(")")))
	{
		OutChange.Type = OutChange.Type.Left(OutChange.Type.Len() - 1);
	}
	OutChange.User = Tokens[8].Mid(0, UserClientIdx);
	OutChange.Client = Tokens[8].Mid(UserClientIdx + 1);

	FString Description;
	for(; LineIdx + 1 < Lines.Num(); LineIdx++)
	{
		if(Lines[LineIdx + 1].Len() == 0)
		{
			Description += TEXT("\n");
		}
		else if(Lines[LineIdx + 1].StartsWith("\t"))
		{
			Description += Lines[LineIdx + 1].Mid(1) + TEXT("\n");
		}
		else
		{
			break;
		}
	}
	Description.TrimStartAndEndInline();

	OutChange.Description = Description;

	return true;
}

bool FPerforceConnection::ConvertToClientPath(const FString& FileName, FString& OutClientFileName, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TSharedPtr<FPerforceWhereRecord> WhereRecord;
	if(Where(FileName, WhereRecord, AbortEvent, Log))
	{
		OutClientFileName = WhereRecord->ClientPath;
		return true;
	}
	else
	{
		OutClientFileName = TEXT("");
		return false;
	}
}

bool FPerforceConnection::ConvertToDepotPath(const FString& FileName, FString& OutDepotFileName, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TSharedPtr<FPerforceWhereRecord> WhereRecord;
	if(Where(FileName, WhereRecord, AbortEvent, Log))
	{
		OutDepotFileName = WhereRecord->DepotPath;
		return true;
	}
	else
	{
		OutDepotFileName = TEXT("");
		return false;
	}
}

bool FPerforceConnection::ConvertToLocalPath(const FString& FileName, FString& OutLocalFileName, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TSharedPtr<FPerforceWhereRecord> WhereRecord;
	if(Where(FileName, WhereRecord, AbortEvent, Log))
	{
		OutLocalFileName = FUtility::GetPathWithCorrectCase(*WhereRecord->LocalPath);
		return true;
	}
	else
	{
		OutLocalFileName = TEXT("");
		return false;
	}
}

bool FPerforceConnection::FindStreams(const FString& Filter, TArray<FString>& OutStreamNames, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<FString> Lines;
	if(!RunCommand(FString::Printf(TEXT("streams -F \"Stream=%s\""), *Filter), Lines, ECommandOptions::None, AbortEvent, Log))
	{
		OutStreamNames.Empty();
		return false;
	}

	TArray<FString> StreamNames;
	for(const FString& Line : Lines)
	{
		TArray<FString> Tokens = Split(Line, TEXT(" "), true);
		if(Tokens.Num() < 2 || Tokens[0] != TEXT("Stream") || !Tokens[1].StartsWith(TEXT("//")))
		{
			Log.Logf(TEXT("Unexpected output from stream query: %s"), *Line);
			OutStreamNames.Empty();
			return false;
		}
		StreamNames.Add(Tokens[1]);
	}
	OutStreamNames = StreamNames;

	return true;
}

bool FPerforceConnection::HasOpenFiles(FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<FPerforceFileRecord> Records;
	bool bResult = RunCommand(TEXT("opened -m 1"), Records, ECommandOptions::None, AbortEvent, Log);
	return bResult && Records.Num() > 0;
}

bool FPerforceConnection::SwitchStream(const FString& NewStream, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommand(FString::Printf(TEXT("client -f -s -S \"%s\" \"%s\""), *NewStream, *ClientName), ECommandOptions::None, AbortEvent, Log);
}

bool FPerforceConnection::Describe(int ChangeNumber, TSharedPtr<FPerforceDescribeRecord>& OutRecord, FEvent* AbortEvent, FOutputDevice& Log) const
{
	FString CommandLine = FString::Printf(TEXT("describe -s %d"), ChangeNumber);

	TArray<TMap<FString, FString>> Records;
	if(!RunCommandWithBinaryOutput(CommandLine, Records, ECommandOptions::None, AbortEvent, Log))
	{
		OutRecord = nullptr;
		return false;
	}
	if(Records.Num() != 1)
	{
		Log.Logf(TEXT("Expected 1 record from p4 %s, got %d"), *CommandLine, Records.Num());
		OutRecord = nullptr;
		return false;
	}

	FString Code;
	if(!TryGetValue(Records[0], TEXT("code"), Code) || Code != "stat")
	{
		Log.Logf(TEXT("Unexpected response from p4 %s (code=%s)"), *CommandLine, *Code);
		OutRecord = nullptr;
		return false;
	}

	OutRecord = MakeShared<FPerforceDescribeRecord>(Records[0]);
	return true;
}

bool FPerforceConnection::Where(const FString& Filter, TSharedPtr<FPerforceWhereRecord>& OutWhereRecord, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<FPerforceFileRecord> FileRecords;
	if(!RunCommand(FString::Printf(TEXT("where \"%s\""), *Filter), FileRecords, ECommandOptions::None, AbortEvent, Log))
	{
		OutWhereRecord = nullptr;
		return false;
	}

	FileRecords.RemoveAll([](const FPerforceFileRecord& Record){ return Record.Unmap; });

	if(FileRecords.Num() == 0)
	{
		Log.Logf(TEXT("'%s' is not mapped to workspace."), *Filter);
		OutWhereRecord = nullptr;
		return false;
	}
	else if(FileRecords.Num() > 1)
	{
		Log.Logf(TEXT("File is mapped to %d locations:"), FileRecords.Num());
		for(const FPerforceFileRecord& FileRecord : FileRecords)
		{
			Log.Logf(TEXT("  %s"), *FileRecord.Path);
		}
		OutWhereRecord = nullptr;
		return false;
	}

	OutWhereRecord = MakeShared<FPerforceWhereRecord>();
	OutWhereRecord->LocalPath = FileRecords[0].Path;
	OutWhereRecord->DepotPath = FileRecords[0].DepotPath;
	OutWhereRecord->ClientPath = FileRecords[0].ClientPath;
	return true;
}

bool FPerforceConnection::Have(const FString& Filter, TArray<FPerforceFileRecord>& OutFileRecords, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommand(FString::Printf(TEXT("have \"%s\""), *Filter), OutFileRecords, ECommandOptions::None, AbortEvent, Log);
}

bool FPerforceConnection::Stat(const FString& Filter, TArray<FPerforceFileRecord>& OutFileRecords, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommand(FString::Printf(TEXT("fstat \"%s\""), *Filter), OutFileRecords, ECommandOptions::None, AbortEvent, Log);
}

bool FPerforceConnection::Sync(const FString& Filter, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommand(FString(TEXT("sync ")) + Filter, ECommandOptions::IgnoreFilesUpToDateError, AbortEvent, Log);
}

bool FPerforceConnection::Sync(const TArray<FString>& DepotPaths, int ChangeNumber, TFunction<void(const FPerforceFileRecord&)> SyncOutput, TArray<FString>& OutTamperedFiles, const FPerforceSyncOptions* Options, FEvent* AbortEvent, FOutputDevice& Log) const
{
	// Write all the files we want to sync to a temp file
	TArray<FString> SyncFiles;
	for(const FString& DepotPath : DepotPaths)
	{
		SyncFiles.Add(FString::Printf(TEXT("%s@%d"), *DepotPath, ChangeNumber));
	}
	FString TempFileName = GetTempFileName();
	FFileHelper::SaveStringArrayToFile(SyncFiles, *TempFileName);

	// Create a filter to strip all the sync records
	FPerforceTagRecordParser Parser([SyncOutput](const TMap<FString, FString>& Tags){ SyncOutput(FPerforceFileRecord(Tags)); });

	FString CommandLine;
	CommandLine += FString::Printf(TEXT("-x \"%s\" -z tag"), *TempFileName);
	if(Options != nullptr && Options->NumRetries > 0)
	{
		CommandLine += FString::Printf(TEXT(" -r %d"), Options->NumRetries);
	}
	if(Options != nullptr && Options->TcpBufferSize > 0)
	{
		CommandLine += FString::Printf(TEXT(" -v net.tcpsize=%d"), Options->TcpBufferSize);
	}
	CommandLine += TEXT(" sync");
	if(Options != nullptr && Options->NumThreads > 1)
	{
		CommandLine += FString::Printf(TEXT(" --parallel=threads=%d"), Options->NumThreads);
	}

	return RunCommand(CommandLine, nullptr, [&Parser, &OutTamperedFiles, &Log](const FPerforceOutputLine& Line){ return FilterSyncOutput(Line, Parser, OutTamperedFiles, Log); }, ECommandOptions::NoFailOnErrors | ECommandOptions::IgnoreFilesUpToDateError | ECommandOptions::IgnoreExitCode, AbortEvent, Log);
}

bool FPerforceConnection::LatestChangeList(int& OutChangeList, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<TMap<FString, FString>> OutLatestChange;
	RunCommand(TEXT("changes -m 1"), OutLatestChange, ECommandOptions::None, AbortEvent, Log);

	if (OutLatestChange.Num() == 1)
	{
		FString StringChangeList;
		if (TryGetValue(OutLatestChange[0], TEXT("change"), StringChangeList))
		{
			return FUtility::TryParse(*StringChangeList, OutChangeList);
		}
	}

	return false;
}

bool FPerforceConnection::FilterSyncOutput(const FPerforceOutputLine& Line, FPerforceTagRecordParser& Parser, TArray<FString>& OutTamperedFiles, FOutputDevice& Log)
{
	if(Line.Channel == EPerforceOutputChannel::TaggedInfo)
	{
		Parser.OutputLine(Line.Text);
		return true;
	}

	Log.Logf(TEXT("%s"), *Line.Text);

	static const FString Prefix = TEXT("Can't clobber writable file ");
	if(Line.Channel == EPerforceOutputChannel::Error && Line.Text.StartsWith(Prefix))
	{
		OutTamperedFiles.Add(Line.Text.Mid(Prefix.Len()).TrimStartAndEnd());
		return true;
	}

	return Line.Channel != EPerforceOutputChannel::Error;
}

void FPerforceConnection::ParseTamperedFile(const FString& Line, TArray<FString>& OutTamperedFiles)
{
	static const FString Prefix = TEXT("Can't clobber writable file ");
	if(Line.StartsWith(Prefix))
	{
		OutTamperedFiles.Add(Line.Mid(Prefix.Len()).TrimStartAndEnd());
	}
}

bool FPerforceConnection::SyncPreview(const FString& Filter, int ChangeNumber, bool bOnlyFilesInThisChange, TArray<FPerforceFileRecord>& OutFileRecords, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommand(FString::Printf(TEXT("sync -n %s@%s%d"), *Filter, bOnlyFilesInThisChange? TEXT("=") : TEXT(""), ChangeNumber), OutFileRecords, ECommandOptions::IgnoreFilesUpToDateError | ECommandOptions::IgnoreNoSuchFilesError, AbortEvent, Log);
}

bool FPerforceConnection::ForceSync(const FString& Filter, int ChangeNumber, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommand(FString::Printf(TEXT("sync -f \"%s\"@%d"), *Filter, ChangeNumber), ECommandOptions::IgnoreFilesUpToDateError, AbortEvent, Log);
}

bool FPerforceConnection::GetOpenFiles(const FString& Filter, TArray<FPerforceFileRecord>& OutFileRecords, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommand(FString::Printf(TEXT("opened \"%s\""), *Filter), OutFileRecords, ECommandOptions::None, AbortEvent, Log);
}

bool FPerforceConnection::GetUnresolvedFiles(const FString& Filter, TArray<FPerforceFileRecord>& OutFileRecords, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommand(FString::Printf(TEXT("fstat -Ru \"%s\""), *Filter), OutFileRecords, ECommandOptions::IgnoreNoSuchFilesError | ECommandOptions::IgnoreFilesNotOpenedOnThisClientError, AbortEvent, Log);
}

bool FPerforceConnection::AutoResolveFile(const FString& File, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommand(FString::Printf(TEXT("resolve -am %s"), *File), ECommandOptions::None, AbortEvent, Log);
}

bool FPerforceConnection::GetActiveStream(FString& OutStreamName, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TSharedPtr<FPerforceSpec> ClientSpec;
	if(TryGetClientSpec(ClientName, ClientSpec, AbortEvent, Log))
	{
		OutStreamName = ClientSpec->GetField(TEXT("Stream"));
		return OutStreamName.Len() > 0;
	}
	else
	{
		OutStreamName.Empty();
		return false;
	}
}

bool FPerforceConnection::RunCommand(const FString& CommandLine, TArray<FPerforceFileRecord>& OutFileRecords, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<TMap<FString, FString>> TagRecords;
	if(!RunCommand(CommandLine, TagRecords, Options, AbortEvent, Log))
	{
		OutFileRecords.Empty();
		return false;
	}
	else
	{
		Algo::Transform(TagRecords, OutFileRecords, [](const TMap<FString, FString>& Tags) -> FPerforceFileRecord { return FPerforceFileRecord(Tags); });
		return true;
	}
}

bool FPerforceConnection::RunCommand(const FString& CommandLine, TArray<FPerforceClientRecord>& OutClientRecords, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<TMap<FString, FString>> TagRecords;
	if(!RunCommand(CommandLine, TagRecords, Options, AbortEvent, Log))
	{
		UE_LOGF(LogUGSCore, Warning, "RunCommand1 returned false");
		return false;
	}
	else
	{
		Algo::Transform(TagRecords, OutClientRecords, [](const TMap<FString, FString>& Tags) -> FPerforceClientRecord { return FPerforceClientRecord(Tags); });
		return true;
	}
}

bool FPerforceConnection::RunCommand(const FString& CommandLine, TArray<TMap<FString, FString>>& OutTagRecords, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log) const
{
	TArray<FString> Lines;
	if(!RunCommand(FString(TEXT("-ztag ")) + CommandLine, EPerforceOutputChannel::TaggedInfo, Lines, Options, AbortEvent, Log))
	//if(!RunCommand(FString(CommandLine, EPerforceOutputChannel::TaggedInfo, Lines, Options, AbortEvent, Log))
	{
		return false;
	}

	FPerforceTagRecordParser Parser([&OutTagRecords](const TMap<FString, FString>& Tags){ OutTagRecords.Add(Tags); });
	for(const FString& Line : Lines)
	{
		Parser.OutputLine(Line);
	}
	return true;
}

bool FPerforceConnection::RunCommand(const FString& CommandLine, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log, const FString& WritePipeText) const
{
	TArray<FString> Lines;
	return RunCommand(CommandLine, Lines, Options, AbortEvent, Log, WritePipeText);
}

bool FPerforceConnection::RunCommand(const FString& CommandLine, TArray<FString>& OutLines, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log, const FString& WritePipeText) const
{
	return RunCommand(CommandLine, EPerforceOutputChannel::Info, OutLines, Options, AbortEvent, Log, WritePipeText);
}

bool FPerforceConnection::RunCommand(const FString& CommandLine, EPerforceOutputChannel Channel, TArray<FString>& OutLines, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log, const FString& WritePipeText) const
{
	FString FullCommandLine = GetFullCommandLine(CommandLine, Options);
	Log.Logf(TEXT("p4> %s %s"), *FPaths::GetCleanFilename(GPerforceExe.GetPerforceExe()), *FullCommandLine);

	TArray<FString> RawOutputLines;

	int ExitCode = GPerforceExe.RunCommand(FullCommandLine, RawOutputLines, AbortEvent, WritePipeText);
	if (ExitCode != 0 && !EnumHasAnyFlags(Options, ECommandOptions::IgnoreExitCode))
	{
		return false;
	}

	bool bResult = true;
	if(EnumHasAnyFlags(Options, ECommandOptions::NoChannels))
	{
		OutLines = RawOutputLines;
	}
	else
	{
		TArray<FString> LocalLines;
		for(const FString& RawOutputLine : RawOutputLines)
		{
			bResult &= ParseCommandOutput(RawOutputLine, [&LocalLines, Channel, &Log](const FPerforceOutputLine& Line){ if(Line.Channel == Channel){ LocalLines.Add(Line.Text); return true; } else { Log.Logf(TEXT("%s"), *Line.Text); return Line.Channel != EPerforceOutputChannel::Error; } }, Options);
		}
		OutLines = LocalLines;
	}
	return bResult;
}

bool FPerforceConnection::RunCommand(const FString& CommandLine, const TCHAR* Input, TFunction<bool(const FPerforceOutputLine&)> HandleOutput, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log) const
{
	FString FullCommandLine = GetFullCommandLine(CommandLine, Options);
	Log.Logf(TEXT("p4> %s %s"), *FPaths::GetCleanFilename(GPerforceExe.GetPerforceExe()), *FullCommandLine);

	TArray<FString> RawOutputLines;

	int ExitCode = GPerforceExe.RunCommand(FullCommandLine, RawOutputLines, AbortEvent);

	// TODO check this vs the old way as things *may* have changed. Before they were handling each line as it was coming
	// back from the process. here we just spin + collect all the output. Then try to process all the output, then if the
	// exit code was not good return false still.
	bool bResult = true;
	FString Line;
	for(const FString& RawOutputLine : RawOutputLines)
	{
		bResult &= ParseCommandOutput(Line, HandleOutput, Options);
	}

	if (ExitCode != 0 && !EnumHasAnyFlags(Options, ECommandOptions::IgnoreExitCode))
	{
		bResult = false;
	}

	return bResult;
}

FString FPerforceConnection::GetFullCommandLine(const FString& CommandLine, ECommandOptions Options) const
{
	FString FullCommandLine;
	if(ServerAndPort.Len() > 0)
	{
		FullCommandLine += FString::Printf(TEXT("-p%s "), *ServerAndPort);
	}
	if(UserName.Len() > 0)
	{
		FullCommandLine += FString::Printf(TEXT("-u%s "), *UserName);
	}
	if((Options & ECommandOptions::NoClient) == ECommandOptions::None && ClientName.Len() > 0)
	{
		FullCommandLine += FString::Printf(TEXT("-c%s "), *ClientName);
	}
	if((Options & ECommandOptions::NoChannels) == ECommandOptions::None)
	{
		FullCommandLine += TEXT("-s ");
	}
	FullCommandLine += CommandLine;

	return FullCommandLine;
}

bool FPerforceConnection::ParseCommandOutput(const FString& Text, TFunction<bool(const FPerforceOutputLine&)> HandleOutput, ECommandOptions Options) const
{
	if(EnumHasAnyFlags(Options, ECommandOptions::NoChannels))
	{
		FPerforceOutputLine Line(EPerforceOutputChannel::Unknown, Text);
		return HandleOutput(Line);
	}
	else if(!IgnoreCommandOutput(Text, Options))
	{
		FPerforceOutputLine Line;
		if(Text.StartsWith(TEXT("text: ")))
		{
			Line = FPerforceOutputLine(EPerforceOutputChannel::Text, Text.Mid(6));
		}
		else if(Text.StartsWith("info: "))
		{
			Line = FPerforceOutputLine(EPerforceOutputChannel::Info, Text.Mid(6));
		}
		else if(Text.StartsWith("info1: "))
		{
			Line = FPerforceOutputLine(IsValidTag(Text, 7)? EPerforceOutputChannel::TaggedInfo : EPerforceOutputChannel::Info, Text.Mid(7));
		}
		else if(Text.StartsWith("warning: "))
		{
			Line = FPerforceOutputLine(EPerforceOutputChannel::Warning, Text.Mid(9));
		}
		else if(Text.StartsWith("error: "))
		{
			Line = FPerforceOutputLine(EPerforceOutputChannel::Error, Text.Mid(7));
		}
		else
		{
			Line = FPerforceOutputLine(EPerforceOutputChannel::Unknown, Text);
		}
		return HandleOutput(Line) && (Line.Channel != EPerforceOutputChannel::Error || EnumHasAnyFlags(Options, ECommandOptions::NoFailOnErrors)) && Line.Channel != EPerforceOutputChannel::Unknown;
	}
	return true;
}

bool FPerforceConnection::IsValidTag(const FString& Line, int StartIndex)
{
	// Annoyingly, we sometimes get commentary with an info1: prefix. Since it typically starts with a depot or file path, we can pick it out.
	for(int Idx = StartIndex; Idx < Line.Len() && Line[Idx] != ' '; Idx++)
	{
		if(Line[Idx] == '/' || Line[Idx] == '\\')
		{
			return false;
		}
	}
	return true;
}

bool FPerforceConnection::IgnoreCommandOutput(const FString& Text, ECommandOptions Options)
{
	if(Text.StartsWith(TEXT("exit: ")) || Text.StartsWith(TEXT("info2: ")) || Text.Len() == 0)
	{
		return true;
	}
	else if(EnumHasAnyFlags(Options, ECommandOptions::IgnoreFilesUpToDateError) && Text.StartsWith(TEXT("error: ")) && Text.EndsWith(TEXT("- file(s) up-to-date.")))
	{
		return true;
	}
	else if(EnumHasAnyFlags(Options, ECommandOptions::IgnoreNoSuchFilesError) && Text.StartsWith(TEXT("error: ")) && Text.EndsWith(TEXT(" - no such file(s).")))
	{
		return true;
	}
	else if(EnumHasAnyFlags(Options, ECommandOptions::IgnoreFilesNotInClientViewError) && Text.StartsWith(TEXT("error: ")) && Text.EndsWith(TEXT("- file(s) not in client view.")))
	{
		return true;
	}
	else if(EnumHasAnyFlags(Options, ECommandOptions::IgnoreFilesNotOpenedOnThisClientError) && Text.StartsWith(TEXT("error: ")) && Text.EndsWith(TEXT(" - file(s) not opened on this client.")))
	{
		return true;
	}
	return false;
}

const uint8* FPerforceConnection::ReadBinaryField(const uint8* Pos, const uint8* End, FString* OutField)
{
	// Read the field type
	if(End - Pos < sizeof(uint8))
	{
		return nullptr;
	}
	uint8 KeyFieldType = *Pos;
	Pos++;

	// Skip over the field data
	if(KeyFieldType == 's')
	{
		// Read the string length
		if(End - Pos < sizeof(uint32))
		{
			return nullptr;
		}
		uint32 KeyLength;
		memcpy(&KeyLength, Pos, sizeof(uint32));
		Pos += sizeof(uint32);

		// Read the string data
		if(End - Pos < KeyLength)
		{
			return nullptr;
		}
		if(OutField != nullptr)
		{
			TStringConversion<FUTF8ToTCHAR_Convert> Converter((const ANSICHAR*)Pos, KeyLength);
			*OutField = FString(Converter.Length(), Converter.Get());
		}
		Pos += KeyLength;
		return Pos;
	}
	else if(KeyFieldType == 'i')
	{
		// Skip over the integer data
		if(End - Pos < sizeof(uint32))
		{
			return nullptr;
		}
		if(OutField != nullptr)
		{
			int32 Value;
			memcpy(&Value, Pos, sizeof(int32));
			*OutField = FString::Printf(TEXT("%d"), Value);
		}
		Pos += sizeof(uint32);
		return Pos;
	}
	else
	{
		checkf(false, TEXT("Invalid field type '%d' (%c)"), (int)*Pos, (TCHAR)*Pos);
		return nullptr;
	}
}

const uint8* FPerforceConnection::ReadBinaryRecord(const uint8* Pos, const uint8* End, TMap<FString, FString>* OutRecord)
{
	if(End > Pos && *Pos == '{')
	{
		Pos++;
		while(Pos < End)
		{
			// If this is the end of the dictionary, return success
			if(*Pos == '0')
			{
				return Pos + 1;
			}

			// Read or skip over the field data
			if(OutRecord == nullptr)
			{
				Pos = ReadBinaryField(Pos, End, nullptr);
				if(Pos == nullptr)
				{
					return nullptr;
				}

				Pos = ReadBinaryField(Pos, End, nullptr);
				if(Pos == nullptr)
				{
					return nullptr;
				}
			}
			else
			{
				FString Key;
				Pos = ReadBinaryField(Pos, End, &Key);
				if(Pos == nullptr)
				{
					return nullptr;
				}

				FString Value;
				Pos = ReadBinaryField(Pos, End, &Value);
				if(Pos == nullptr)
				{
					return nullptr;
				}

				OutRecord->FindOrAdd(MoveTemp(Key)) = MoveTemp(Value);
			}
		}
	}
	return nullptr;
}

/// <summary>
/// Execute a Perforce command and parse the output as marshalled Python objects. This is more robustly defined than the text-based tagged output
/// format, because it avoids ambiguity when returned fields can have newlines.
/// </summary>
/// <param name="CommandLine">Command line to execute Perforce with</param>
/// <param name="TaggedOutput">List that receives the output records</param>
/// <param name="WithClient">Whether to include client information on the command line</param>
bool FPerforceConnection::RunCommandWithBinaryOutput(const FString& CommandLine, TArray<TMap<FString, FString>>& OutRecords, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log) const
{
	return RunCommandWithBinaryOutput(CommandLine, [&OutRecords](const TMap<FString, FString>& Record){ OutRecords.Add(Record); return true; }, Options, AbortEvent, Log);
}

/// <summary>
/// Execute a Perforce command and parse the output as marshalled Python objects. This is more robustly defined than the text-based tagged output
/// format, because it avoids ambiguity when returned fields can have newlines.
/// </summary>
/// <param name="CommandLine">Command line to execute Perforce with</param>
/// <param name="TaggedOutput">List that receives the output records</param>
/// <param name="WithClient">Whether to include client information on the command line</param>
bool FPerforceConnection::RunCommandWithBinaryOutput(const FString& CommandLine, TFunction<void(const TMap<FString, FString>&)> HandleOutput, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log) const
{
	FString FullCommandLine(TEXT("-G ") + CommandLine);

	uint32 ProcId;
	void* ChildWritePipe  = nullptr;
	void* ParentReadPipe = nullptr;
	FPlatformProcess::CreatePipe(ParentReadPipe, ChildWritePipe);

	FProcHandle P4Proc = FPlatformProcess::CreateProc(*GPerforceExe.GetPerforceExe(), *FullCommandLine, false, true, true, &ProcId, 0, nullptr, ChildWritePipe);

	size_t BufferPos = 0;
	size_t BufferEnd = 0;
	TArray<uint8> Buffer;
	TMap<FString, FString> Record;
	for(;;)
	{
		// Determine if the process has exited. Do this before reading, so we'll only exit the loop if no data is read AND the process has exited.
		bool bHasExited = !FPlatformProcess::IsProcRunning(P4Proc);

		// Check that we don't have an abort event
		if (AbortEvent->Wait(FTimespan::Zero()))
		{
			FPlatformProcess::TerminateProc(P4Proc);
			return false;
			//throw FAbortException();
		}

		// Read data from the child process
		TArray<uint8> TempBuffer;
		bool bRead = FPlatformProcess::ReadPipeToArray(ParentReadPipe, TempBuffer);
		size_t BytesRead = TempBuffer.Num();

		// Add new data to the end of the Buffer
		Buffer += TempBuffer;

		if(BytesRead == 0)
		{
			// If it exited, quit. Otherwise sleep until data is available.
			if(bHasExited)
			{
				break;
			}
			else if(AbortEvent->Wait(FTimespan::FromMilliseconds(50)))
			{
				FPlatformProcess::TerminateProc(P4Proc);
				return false;
				//throw FAbortException();
			}
		}
		else
		{
			// Add the read bytes to the buffer
			BufferEnd += BytesRead;

			// Process all the records in the buffer that we can
			const uint8* RecordPos = Buffer.GetData() + BufferPos;
			const uint8* RecordMaxPos = Buffer.GetData() + BufferEnd;
			for(;;)
			{
				const uint8* RecordEnd = ReadBinaryRecord(RecordPos, RecordMaxPos, &Record);
				if(RecordEnd == nullptr)
				{
					break;
				}
				HandleOutput(Record);
				RecordPos = RecordEnd;
			}
			BufferPos = RecordPos - Buffer.GetData();

			// Shrink the buffer down if we've got spare space
			if(BufferPos > 64)
			{
				BufferEnd -= BufferPos;
				memmove(Buffer.GetData(), Buffer.GetData() + BufferPos, BufferEnd);
				BufferPos = 0;
			}
		}
	}

	FPlatformProcess::ClosePipe(ParentReadPipe, ChildWritePipe);

	int ExitCode = -1;

	checkf(BufferPos == BufferEnd, TEXT("%zu bytes of incomplete record data received from P4"), BufferEnd - BufferPos);
	FPlatformProcess::GetProcReturnCode(P4Proc, &ExitCode);

	return ExitCode == 0;
}

void FPerforceConnection::OpenP4V(const FString& AdditionalArgs) const
{
#if PLATFORM_WINDOWS
	const TCHAR* P4VExe = TEXT("p4v.exe");
#else
	const TCHAR* P4VExe = TEXT("p4v");
#endif

	FString P4VArgs = GetArgumentsForExternalProgram() + TEXT(" ") + AdditionalArgs;

	FPlatformProcess::CreateProc(P4VExe, *P4VArgs, true, true, true, nullptr, 0, nullptr, nullptr);
}

void FPerforceConnection::OpenP4VC(const FString& AdditionalArgs) const
{
#if PLATFORM_WINDOWS
	const TCHAR* P4VCExe = TEXT("p4vc.exe");
#else
	const TCHAR* P4VCExe = TEXT("p4vc");
#endif

	FString P4VCArgs = GetArgumentsForExternalProgram() + TEXT(" ") + AdditionalArgs;

	FPlatformProcess::CreateProc(P4VCExe, *P4VCArgs, true, true, true, nullptr, 0, nullptr, nullptr);
}

FString FPerforceConnection::GetArgumentsForExternalProgram(bool bIncludeClient) const
{
	FString ExternalArgs = FString::Printf(TEXT("-p \"%s\" -u \"%s\""), *ServerAndPort, *UserName);

	if (bIncludeClient && !ClientName.IsEmpty())
	{
		ExternalArgs += FString::Printf(TEXT(" -c \"%s\""), *ClientName);
	}

	return ExternalArgs;
}

//// FPerforceUtils ////

FString FPerforceUtils::GetClientOrDepotDirectoryName(const TCHAR* ClientFile)
{
	const TCHAR* LastSlash = FCString::Strrchr(ClientFile, '/');
	if(LastSlash == nullptr)
	{
		return "";
	}
	else
	{
		return FString(LastSlash - ClientFile, ClientFile);
	}
}

bool FPerforceUtils::TryParseDateTime(const FString& Date, const FString& Time, FDateTime& OutDate)
{
	// Parse the date
	const TCHAR* Pos = *Date;
	TCHAR* End = nullptr;

	int32 Year = FCString::Strtoi(Pos, &End, 10);
	if(End <= Pos || *End != '/')
	{
		return false;
	}
	Pos = End + 1;

	int32 Month = FCString::Strtoi(Pos, &End, 10);
	if(End <= Pos || *End != '/')
	{
		return false;
	}
	Pos = End + 1;

	int32 Day = FCString::Strtoi(Pos, &End, 10);
	if(End <= Pos || *End != 0)
	{
		return false;
	}

	// Parse the time
	Pos = *Time;

	int Hour = FCString::Strtoi(Pos, &End, 10);
	if(End <= Pos || *End != ':')
	{
		return false;
	}
	Pos = End  + 1;

	int Minute = FCString::Strtoi(Pos, &End, 10);
	if(End <= Pos || *End != ':')
	{
		return false;
	}
	Pos = End + 1;

	int Second = FCString::Strtoi(Pos, &End, 10);
	if(End <= Pos || *End != 0)
	{
		return false;
	}

	OutDate = FDateTime(Year, Month, Day, Hour, Minute, Second);
	return true;
}

} // namespace UGSCore
