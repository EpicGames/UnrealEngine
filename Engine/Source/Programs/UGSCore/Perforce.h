// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Tuple.h"

namespace UGSCore
{

struct FPerforceChangeSummary
{
	int Number;
	FDateTime Date;
	FString User;
	FString Client;
	FString Description;

	FPerforceChangeSummary();
};

struct FPerforceFileChangeSummary
{
	int Revision;
	int ChangeNumber;
	FString Action;
	FDateTime Date;
	FString User;
	FString Client;
	FString Type;
	FString Description;

	FPerforceFileChangeSummary();
};

struct FPerforceClientRecord
{
	FString Name;
	FString Owner;
	FString Host;
	FString Root;

	FPerforceClientRecord(const TMap<FString, FString>& Tags);
};

struct FPerforceDescribeFileRecord
{
	FString DepotFile;
	FString Action;
	FString Type;
	int Revision;
	int FileSize;
	FString Digest;

	FPerforceDescribeFileRecord();
};

struct FPerforceDescribeRecord
{
	int ChangeNumber;
	FString User;
	FString Client;
	int64 Time;
	FString Description;
	FString Status;
	FString ChangeType;
	FString Path;
	TArray<FPerforceDescribeFileRecord> Files;

	FPerforceDescribeRecord(const TMap<FString, FString>& Tags);
};

struct FPerforceInfoRecord
{
	FString UserName;
	FString HostName;
	FString ClientAddress;
	FString ServerAddress;
	FTimespan ServerTimeZone;

	FPerforceInfoRecord(const TMap<FString, FString>& Tags);
};

struct FPerforceFileRecord
{
	FString DepotPath;
	FString ClientPath;
	FString Path;
	FString Action;
	int Revision;
	bool IsMapped;
	bool Unmap;

	FPerforceFileRecord(const TMap<FString, FString>& Tags);
};

struct FPerforceTagRecordParser
{
	TFunction<void(const TMap<FString, FString>&)> OutputRecord;
	TMap<FString, FString> Tags;

	FPerforceTagRecordParser(TFunction<void(const TMap<FString, FString>&)> InOutputRecord);
	~FPerforceTagRecordParser();

	void OutputLine(const FString& Line);
	void Flush();
};

struct FPerforceWhereRecord
{
	FString LocalPath;
	FString ClientPath;
	FString DepotPath;
};

struct FPerforceSyncOptions
{
	int NumRetries;
	int NumThreads;
	int TcpBufferSize;

	FPerforceSyncOptions();
};

struct FPerforceSpec
{
	TArray<TTuple<FString, FString>> Sections;

	/**
	 * Gets the current value of a field with the given name 
	 *
	 * @param Name Name of the field to search for
	 * @return The value of the field, or null if it does not exist.
	 */
	const TCHAR* GetField(const TCHAR* Name) const;

	/**
	 * Sets the value of an existing field, or adds a new one with the given name
	 *
	 * @param Name of the field to set
	 * @param New value of the field
	 */
	void SetField(const TCHAR* Name, const TCHAR* Value);

	/**
	 * Parses a spec (clientspec, branchspec, changespec) from an array of lines
	 *
	 * @param Lines Text split into separate lines
	 * @return Array of section names and values
	 */
	static bool TryParse(const TArray<FString>& Lines, TSharedPtr<FPerforceSpec>& OutSpec, FOutputDevice& Log);

	/**
	 * Formats a P4 specification as a block of text
	 *
	 * @return Spec as a single string
	 */
	FString ToString() const;
};

enum class EPerforceOutputChannel
{
	Unknown,
	Text,
	Info,
	TaggedInfo,
	Warning,
	Error,
	Exit
};

struct FPerforceOutputLine
{
	EPerforceOutputChannel Channel;
	FString Text;

	FPerforceOutputLine();
	FPerforceOutputLine(EPerforceOutputChannel InChannel, const FString& InText);
};

class FPerforceConnection
{
public:
	enum class ECommandOptions
	{
		None = 0x0,
		NoClient = 0x1,
		NoFailOnErrors = 0x2,
		NoChannels = 0x4,
		IgnoreFilesUpToDateError = 0x8,
		IgnoreNoSuchFilesError = 0x10,
		IgnoreFilesNotOpenedOnThisClientError = 0x20,
		IgnoreExitCode = 0x40,
		IgnoreFilesNotInClientViewError = 0x80,
	};

	const FString ServerAndPort;
	const FString UserName;
	const FString ClientName;

	FPerforceConnection(const TCHAR* InUserName, const TCHAR* InClientName, const TCHAR* InServerAndPort);
	TSharedRef<FPerforceConnection> OpenClient(const FString& NewClientName) const;

	bool Info(TSharedPtr<FPerforceInfoRecord>& OutInfo, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool GetSetting(const FString& Name, FString& Value, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool FindClients(TArray<FPerforceClientRecord>& Clients, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool FindClients(TArray<FPerforceClientRecord>& Clients, const FString& ForUserName, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool CreateClient(const FPerforceClientRecord& Client, const FString& Stream, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool TryGetClientSpec(const FString& ClientName, TSharedPtr<FPerforceSpec>& OutSpec, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool TryGetStreamSpec(const FString& StreamName, TSharedPtr<FPerforceSpec>& OutSpec, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool FindFiles(const FString& Filter, TArray<FPerforceFileRecord>& FileRecords, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool Print(const FString& DepotPath, TArray<FString>& Lines, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool PrintToFile(const FString& DepotPath, const FString& OutputFileName, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool FileExists(const FString& Filter, bool& bExists, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool FindChanges(const FString& Filter, int MaxResults, TArray<FPerforceChangeSummary>& OutChanges, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool FindChanges(const TArray<FString>& Filters, int MaxResults, TArray<FPerforceChangeSummary>& OutChanges, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool TryParseChangeSummary(const FString& Line, FPerforceChangeSummary& Change) const;
	bool FindFileChanges(const FString& FilePath, int MaxResults, TArray<FPerforceFileChangeSummary>& Changes, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool TryParseFileChangeSummary(const TArray<FString>& Lines, int& LineIdx, FPerforceFileChangeSummary& OutChange) const;
	bool ConvertToClientPath(const FString& FileName, FString& OutClientFileName, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool ConvertToDepotPath(const FString& FileName, FString& OutDepotFileName, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool ConvertToLocalPath(const FString& FileName, FString& OutLocalFileName, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool FindStreams(const FString& Filter, TArray<FString>& OutStreamNames, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool HasOpenFiles(FEvent* AbortEvent, FOutputDevice& Log) const;
	bool SwitchStream(const FString& NewStream, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool Describe(int ChangeNumber, TSharedPtr<FPerforceDescribeRecord>& OutRecord, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool Where(const FString& Filter, TSharedPtr<FPerforceWhereRecord>& OutWhereRecord, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool Have(const FString& Filter, TArray<FPerforceFileRecord>& OutFileRecords, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool Stat(const FString& Filter, TArray<FPerforceFileRecord>& OutFileRecords, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool Sync(const FString& Filter, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool Sync(const TArray<FString>& DepotPaths, int ChangeNumber, TFunction<void(const FPerforceFileRecord&)> SyncOutput, TArray<FString>& OutTamperedFiles, const FPerforceSyncOptions* Options, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool LatestChangeList(int& OutChangeList, FEvent* AbortEvent, FOutputDevice& Log) const;

	void OpenP4V(const FString& AdditionalArgs = FString()) const;
	void OpenP4VC(const FString& AdditionalArgs = FString()) const;

private:
	static bool FilterSyncOutput(const FPerforceOutputLine& Line, FPerforceTagRecordParser& Parser, TArray<FString>& OutTamperedFiles, FOutputDevice& Log);
	static void ParseTamperedFile(const FString& Line, TArray<FString>& OutTamperedFiles);

public:
	bool SyncPreview(const FString& Filter, int ChangeNumber, bool bOnlyFilesInThisChange, TArray<FPerforceFileRecord>& OutFileRecords, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool ForceSync(const FString& Filter, int ChangeNumber, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool GetOpenFiles(const FString& Filter, TArray<FPerforceFileRecord>& OutFileRecords, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool GetUnresolvedFiles(const FString& Filter, TArray<FPerforceFileRecord>& OutFileRecords, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool AutoResolveFile(const FString& Filter, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool GetActiveStream(FString& OutStreamName, FEvent* AbortEvent, FOutputDevice& Log) const;

private:
	bool RunCommand(const FString& CommandLine, TArray<FPerforceFileRecord>& OutFileRecords, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool RunCommand(const FString& CommandLine, TArray<FPerforceClientRecord>& OutClientRecords, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool RunCommand(const FString& CommandLine, TArray<TMap<FString, FString>>& OutTagRecords, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool RunCommand(const FString& CommandLine, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log, const FString& WritePipeText = FString()) const;
	bool RunCommand(const FString& CommandLine, TArray<FString>& OutLines, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log, const FString& WritePipeText = FString()) const;
	bool RunCommand(const FString& CommandLine, EPerforceOutputChannel Channel, TArray<FString>& OutLines, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log, const FString& WritePipeText = FString()) const;
	bool RunCommand(const FString& CommandLine, const TCHAR* Input, TFunction<bool(const FPerforceOutputLine&)> HandleOutput, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log) const;
	FString GetFullCommandLine(const FString& CommandLine, ECommandOptions Options) const;
	bool ParseCommandOutput(const FString& Text, TFunction<bool(const FPerforceOutputLine&)> HandleOutput, ECommandOptions Options) const;
	static bool IsValidTag(const FString& Line, int StartIndex);
	static bool IgnoreCommandOutput(const FString& Text, ECommandOptions Options);

	static const uint8* ReadBinaryField(const uint8* Pos, const uint8* End, FString* OutField);
	static const uint8* ReadBinaryRecord(const uint8* Pos, const uint8* End, TMap<FString, FString>* OutRecord);

	bool RunCommandWithBinaryOutput(const FString& CommandLine, TArray<TMap<FString, FString>>& OutRecords, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log) const;
	bool RunCommandWithBinaryOutput(const FString& CommandLine, TFunction<void(const TMap<FString, FString>&)> HandleOutput, ECommandOptions Options, FEvent* AbortEvent, FOutputDevice& Log) const;

	FString GetArgumentsForExternalProgram(bool bIncludeClient = true) const;
};

ENUM_CLASS_FLAGS(FPerforceConnection::ECommandOptions)

//// FPerforceUtils ////

struct FPerforceUtils
{
	static FString GetClientOrDepotDirectoryName(const TCHAR* ClientFile);
	static bool TryParseDateTime(const FString& Date, const FString& Time, FDateTime& OutDate);
};

} // namespace UGSCore
