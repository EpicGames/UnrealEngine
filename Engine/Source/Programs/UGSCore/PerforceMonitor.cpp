// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerforceMonitor.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "HAL/Event.h"
#include "Misc/ScopeLock.h"
#include "CustomConfigFile.h"
#include "Utility.h"

namespace UGSCore
{

FPerforceMonitor::FPerforceMonitor(const TSharedRef<FPerforceConnection>& InPerforce, const FString& InBranchClientPath, const FString& InSelectedClientFileName, const FString& InSelectedProjectIdentifier, const FString& InLogPath)
	: Perforce(InPerforce)
	, BranchClientPath(InBranchClientPath)
	, SelectedClientFileName(InSelectedClientFileName)
	, SelectedProjectIdentifier(InSelectedProjectIdentifier)
	, LogWriter(*InLogPath, 128 * 1024)
	, WorkerThread(nullptr)
	, ZippedBinariesConfigChangeNumber(0)
	, RefreshEvent(FPlatformProcess::GetSynchEventFromPool())
	, AbortEvent(FPlatformProcess::GetSynchEventFromPool(true))
	, CurrentMaxChanges(0)
	, PendingMaxChanges(100)
	, LastChangeByCurrentUser(-1)
	, LastCodeChangeByCurrentUser(-1)
	, bDisposing(false)
{
}

FPerforceMonitor::~FPerforceMonitor()
{
	bDisposing = true;
	if(WorkerThread != nullptr)
	{
		AbortEvent->Trigger();
		RefreshEvent->Trigger();

		WorkerThread->WaitForCompletion();
		WorkerThread = nullptr;
	}
	FPlatformProcess::ReturnSynchEventToPool(RefreshEvent);
}

void FPerforceMonitor::Start()
{
	check(WorkerThread == nullptr);
	WorkerThread = FRunnableThread::Create(this, *FString::Printf(TEXT("PerforceMonitor-%s"), *SelectedClientFileName));
}

FString FPerforceMonitor::GetLastStatusMessage() const
{
	FScopeLock Lock(&CriticalSection);
	return LastStatusMessage;
}

int FPerforceMonitor::GetCurrentMaxChanges() const
{
	FScopeLock Lock(&CriticalSection);
	return CurrentMaxChanges;
}

int FPerforceMonitor::GetPendingMaxChanges() const
{
	FScopeLock Lock(&CriticalSection);
	return PendingMaxChanges;
}

TArray<FString> FPerforceMonitor::GetOtherStreamNames() const
{
	FScopeLock Lock(&CriticalSection);
	return OtherStreamNames;
}

int FPerforceMonitor::GetLastChangeByCurrentUser() const
{
	FScopeLock Lock(&CriticalSection);
	return LastChangeByCurrentUser;
}

int FPerforceMonitor::GetLastCodeChangeByCurrentUser() const
{
	FScopeLock Lock(&CriticalSection);
	return LastCodeChangeByCurrentUser;
}

bool FPerforceMonitor::HasZippedBinaries() const
{
	FScopeLock Lock(&CriticalSection);
	return ChangeNumberToZippedBinaries.Num() > 0;
}

TArray<TSharedRef<FPerforceChangeSummary, ESPMode::ThreadSafe>> FPerforceMonitor::GetChanges() const
{
	FScopeLock Lock(&CriticalSection);
	return Changes;
}

bool FPerforceMonitor::TryGetChangeType(int ChangeNumber, FChangeType& OutType) const
{
	FScopeLock Lock(&CriticalSection);

	const FChangeType* ChangeType = ChangeNumberToType.Find(ChangeNumber);
	if(ChangeType != nullptr)
	{
		OutType = *ChangeType;
		return true;
	}

	return false;
}

bool FPerforceMonitor::TryGetArchivePathForChangeNumber(int ChangeNumber, FString& OutArchivePath) const
{
	FScopeLock Lock(&CriticalSection);

	const FString* ArchivePath = ChangeNumberToZippedBinaries.Find(ChangeNumber);
	if(ArchivePath != nullptr)
	{
		OutArchivePath = *ArchivePath;
		return true;
	}
	return false;
}

TSet<int> FPerforceMonitor::GetPromotedChangeNumbers() const
{
	FScopeLock Lock(&CriticalSection);
	return PromotedChangeNumbers;
}

void FPerforceMonitor::Refresh()
{
	RefreshEvent->Trigger();
}

uint32 FPerforceMonitor::Run()
{
	// TODO we should likely turn on exceptions, or handle this better
	//try
	{
		RunInternal();
	}
	//catch(FAbortException)
	{
	}
	return 0;
}

void FPerforceMonitor::RunInternal()
{
	FString StreamName;
	Perforce->GetActiveStream(StreamName, AbortEvent, LogWriter);

	// Try to update the zipped binaries list before anything else, because it causes a state change in the UI
	UpdateZippedBinaries();

	while(!bDisposing)
	{
		FDateTime StartTime = FDateTime::UtcNow();

		// Check we haven't switched streams
		if(OnStreamChange)
		{
			FString NewStreamName;
			if(Perforce->GetActiveStream(NewStreamName, AbortEvent, LogWriter) && NewStreamName != StreamName)
			{
				OnStreamChange();
			}
		}

		// Update the stream list
		if(StreamName.Len() > 0)
		{
			TArray<FString> NewOtherStreamNames;
			Perforce->FindStreams(FPerforceUtils::GetClientOrDepotDirectoryName(*StreamName) + "/*", NewOtherStreamNames, AbortEvent, LogWriter);

			FScopeLock Lock(&CriticalSection);
			OtherStreamNames = NewOtherStreamNames;
		}

		// Check for any p4 changes
		if(!UpdateChanges())
		{
			LastStatusMessage = TEXT("Failed to update changes");
		}
		else if(!UpdateChangeTypes())
		{
			LastStatusMessage = TEXT("Failed to update change types");
		}
		else if(!UpdateZippedBinaries())
		{
			LastStatusMessage = TEXT("Failed to update zipped binaries list");
		}
		else
		{
			LastStatusMessage = FString::Printf(TEXT("Last update took %dms"), int((FDateTime::UtcNow() - StartTime).GetTotalMilliseconds()));
		}

		// Wait for another request, or scan for new builds after a timeout
		RefreshEvent->Wait(FTimespan::FromSeconds(60.0));
	}
}

bool FPerforceMonitor::UpdateChanges()
{
	// Get the current status of the build
	int MaxChanges;
	int OldestChangeNumber = -1;
	int NewestChangeNumber = -1;
	TSet<int> CurrentChangelists;
	TSet<int> PrevPromotedChangelists;
	{
		FScopeLock Lock(&CriticalSection);
		MaxChanges = PendingMaxChanges;
		if(Changes.Num() > 0)
		{
			NewestChangeNumber = Changes[0]->Number;
			OldestChangeNumber = Changes[Changes.Num() - 1]->Number;
		}
		for(const TSharedRef<FPerforceChangeSummary, ESPMode::ThreadSafe>& Change : Changes)
		{
			CurrentChangelists.Add(Change->Number);
		}
		PrevPromotedChangelists = PromotedChangeNumbers;
	}

	// Build a full list of all the paths to sync
	TArray<FString> DepotPaths;
	if(SelectedClientFileName.EndsWith(TEXT(".uprojectdirs")))
	{
		DepotPaths.Add(FString::Printf(TEXT("%s/..."), *BranchClientPath));
	}
	else
	{
		DepotPaths.Add(FString::Printf(TEXT("%s/*"), *BranchClientPath));
		DepotPaths.Add(FString::Printf(TEXT("%s/Engine/..."), *BranchClientPath));
		DepotPaths.Add(FString::Printf(TEXT("%s/..."), *FPerforceUtils::GetClientOrDepotDirectoryName(*SelectedClientFileName)));
	}

	// Read any new changes
	TArray<FPerforceChangeSummary> NewChanges;
	if(MaxChanges > CurrentMaxChanges)
	{
		if(!Perforce->FindChanges(DepotPaths, MaxChanges, NewChanges, AbortEvent, LogWriter))
		{
			return false;
		}
	}
	else
	{
		TArray<FString> RangedDepotPaths;
		for(const FString& DepotPath : DepotPaths)
		{
			RangedDepotPaths.Add(FString::Printf(TEXT("%s@>%d"), *DepotPath, NewestChangeNumber));
		}
		if(!Perforce->FindChanges(RangedDepotPaths, -1, NewChanges, AbortEvent, LogWriter))
		{
			return false;
		}
	}

	// Remove anything we already have
	NewChanges.RemoveAll([&CurrentChangelists](const FPerforceChangeSummary& Change){ return CurrentChangelists.Contains(Change.Number); });

	// Update the change ranges
	if(NewChanges.Num() > 0)
	{
		OldestChangeNumber = FMath::Max(OldestChangeNumber, NewChanges[NewChanges.Num() - 1].Number);
		NewestChangeNumber = FMath::Min(NewestChangeNumber, NewChanges[0].Number);
	}

	// Fixup any ROBOMERGE authors
	const FString RoboMergePrefix = TEXT("#ROBOMERGE-AUTHOR:");
	for(FPerforceChangeSummary& Change : NewChanges)
	{
		if(Change.Description.StartsWith(RoboMergePrefix))
		{
			int StartIdx = RoboMergePrefix.Len();
			while(StartIdx < Change.Description.Len() && Change.Description[StartIdx] == ' ')
			{
				StartIdx++;
			}

			int EndIdx = StartIdx;
			while(EndIdx < Change.Description.Len() && !FChar::IsWhitespace(Change.Description[EndIdx]))
			{
				EndIdx++;
			}

			if(EndIdx > StartIdx)
			{
				Change.User = Change.Description.Mid(StartIdx, EndIdx - StartIdx);
				Change.Description = "ROBOMERGE: " + Change.Description.Mid(EndIdx).TrimStartAndEnd();
			}
		}
	}

	// Process the new changes received
	if(NewChanges.Num() > 0 || MaxChanges < CurrentMaxChanges)
	{
		// Insert them into the builds list
		{
			FScopeLock Lock(&CriticalSection);

			int InsertIdx = 0;
			for(const FPerforceChangeSummary& NewChange : NewChanges)
			{
				while(InsertIdx < Changes.Num() && NewChange.Number < Changes[InsertIdx]->Number)
				{
					InsertIdx++;
				}
				Changes.Insert(MakeShared<FPerforceChangeSummary, ESPMode::ThreadSafe>(NewChange), InsertIdx);
			}

			if(Changes.Num() > MaxChanges)
			{
				Changes.RemoveAt(MaxChanges, Changes.Num() - MaxChanges);
			}

			CurrentMaxChanges = MaxChanges;
		}

		// Find the last submitted change by the current user
		int NewLastChangeByCurrentUser = -1;
		for(const FChangeSharedRef& Change : Changes)
		{
			if(Change->User == Perforce->UserName)
			{
				NewLastChangeByCurrentUser = FMath::Max(NewLastChangeByCurrentUser, Change->Number);
			}
		}
		LastChangeByCurrentUser = NewLastChangeByCurrentUser;

		// Notify the main window that we've got more data
		if(OnUpdate)
		{
			OnUpdate();
		}
	}
	return true;
}

bool FPerforceMonitor::UpdateChangeTypes()
{
	// Find the changes we need to query
	TArray<int> QueryChangeNumbers;
	{
		FScopeLock Lock(&CriticalSection);
		for(const FChangeSharedRef& Change : Changes)
		{
			if(!ChangeNumberToType.Contains(Change->Number))
			{
				QueryChangeNumbers.Add(Change->Number);
			}
		}
	}

	// Update them in batches
	for(int QueryChangeNumber : QueryChangeNumbers)
	{
		// If there's something to check for, find all the content changes after this changelist
		TSharedPtr<FPerforceDescribeRecord> DescribeRecord;
		if (Perforce->Describe(QueryChangeNumber, DescribeRecord, AbortEvent, LogWriter))
		{
			FChangeType Type;

			for (const FPerforceDescribeFileRecord& File : DescribeRecord->Files)
			{
				const FString& DepotFile = File.DepotFile;

				// TODO move this to a map, or something, way better way to do this then whats going on here
				if (DepotFile.EndsWith(TEXT(".c")) || DepotFile.EndsWith(TEXT(".cc")) || DepotFile.EndsWith(TEXT(".cpp")) ||
					DepotFile.EndsWith(TEXT(".inl")) || DepotFile.EndsWith(TEXT(".m")) || DepotFile.EndsWith(TEXT(".mm")) ||
					DepotFile.EndsWith(TEXT(".rc")) || DepotFile.EndsWith(TEXT(".cs")) || DepotFile.EndsWith(TEXT(".csproj")) ||
					DepotFile.EndsWith(TEXT(".h")) || DepotFile.EndsWith(TEXT(".hpp")) || DepotFile.EndsWith(TEXT(".inl")) ||
					DepotFile.EndsWith(TEXT(".usf")) || DepotFile.EndsWith(TEXT(".ush")) || DepotFile.EndsWith(TEXT(".uproject")) ||
					DepotFile.EndsWith(TEXT(".uplugin")) || DepotFile.EndsWith(TEXT(".sln")))
				{
					Type.bContainsCode = true;
				}
				else
				{
					Type.bContainsContent = true;
				}

				if (Type.bContainsCode && Type.bContainsContent)
				{
					break;
				}
			}

			// Update the type of this change
			{
				FScopeLock Lock(&CriticalSection);
				if(!ChangeNumberToType.Contains(QueryChangeNumber))
				{
					ChangeNumberToType.Add(QueryChangeNumber, Type);
				}
			}
		}

		// Find the last submitted code change by the current user
		int NewLastCodeChangeByCurrentUser = -1;
		for(const FChangeSharedRef& Change : Changes)
		{
			if(Change->User == Perforce->UserName)
			{
				FChangeType* Type = ChangeNumberToType.Find(Change->Number);
				if(Type != nullptr && Type->bContainsCode)
				{
					NewLastCodeChangeByCurrentUser = FMath::Max(NewLastCodeChangeByCurrentUser, Change->Number);
				}
			}
		}
		LastCodeChangeByCurrentUser = NewLastCodeChangeByCurrentUser;

		// Notify the main window that we've got an update
		if(OnUpdateMetadata)
		{
			OnUpdateMetadata();
		}
	}

	if (OnChangeTypeQueryFinished)
	{
		OnChangeTypeQueryFinished();
	}

	return true;
}

bool FPerforceMonitor::UpdateZippedBinaries()
{
	// Get the path to the config file
	FString ClientConfigFileName = FPerforceUtils::GetClientOrDepotDirectoryName(*SelectedClientFileName) + TEXT("/Build/UnrealGameSync.ini");

	// Find the most recent change to that file (if the file doesn't exist, we succeed and just get 0 changes).
	TArray<FPerforceChangeSummary> ConfigFileChanges;
	if(!Perforce->FindChanges(ClientConfigFileName, 1, ConfigFileChanges, AbortEvent, LogWriter))
	{
		return false;
	}

	// Update the zipped binaries path if it's changed
	int NewZippedBinariesConfigChangeNumber = (ConfigFileChanges.Num() > 0)? ConfigFileChanges[0].Number : 0;
	if(NewZippedBinariesConfigChangeNumber != ZippedBinariesConfigChangeNumber)
	{
		// Read the config file
		FString NewZippedBinariesPath;
		if(NewZippedBinariesConfigChangeNumber != 0)
		{
			TArray<FString> Lines;
			if(!Perforce->Print(ClientConfigFileName, Lines, AbortEvent, LogWriter))
			{
				return false;
			}

			FCustomConfigFile NewConfigFile;
			NewConfigFile.Parse(Lines);

			TSharedPtr<const FCustomConfigSection> ProjectSection = NewConfigFile.FindSection(*SelectedProjectIdentifier);
			if(ProjectSection.IsValid())
			{
				NewZippedBinariesPath = ProjectSection->GetValue(TEXT("ZippedBinariesPath"), TEXT(""));
			}
		}

		// Update the stored settings
		{
			FScopeLock Lock(&CriticalSection);
			ZippedBinariesPath = NewZippedBinariesPath;
			ZippedBinariesConfigChangeNumber = NewZippedBinariesConfigChangeNumber;
		}
	}

	TMap<int, FString> NewChangeNumberToZippedBinaries;
	if(ZippedBinariesPath.Len() > 0)
	{
		TArray<FPerforceFileChangeSummary> ZipChanges;
		if(!Perforce->FindFileChanges(ZippedBinariesPath, 100, ZipChanges, AbortEvent, LogWriter))
		{
			return false;
		}

		for(const FPerforceFileChangeSummary& ZipChange : ZipChanges)
		{
			if(ZipChange.Action != TEXT("purge"))
			{
				TArray<FString> Tokens;
				ZipChange.Description.ParseIntoArray(Tokens, TEXT(" "));
				if(Tokens.Num() >= 2 && Tokens[0].StartsWith(TEXT("[CL")) && Tokens[1].EndsWith(TEXT("]")))
				{
					int OriginalChangeNumber;
					if(FUtility::TryParse(*Tokens[1].Mid(0, Tokens[1].Len() - 1), OriginalChangeNumber) && !NewChangeNumberToZippedBinaries.Contains(OriginalChangeNumber))
					{
						NewChangeNumberToZippedBinaries.Add(OriginalChangeNumber, FString::Printf(TEXT("%s#%d"), *ZippedBinariesPath, ZipChange.Revision));
					}
				}
			}
		}
	}

	// Check if the list of zipped binaries have changed
	{
		FScopeLock Lock(&CriticalSection);

		bool bUpdate = ChangeNumberToZippedBinaries.Num() != NewChangeNumberToZippedBinaries.Num();
		if(!bUpdate)
		{
			for(const TTuple<int, FString>& Pair : ChangeNumberToZippedBinaries)
			{
				const FString* Other = NewChangeNumberToZippedBinaries.Find(Pair.Key);
				if(Other == nullptr || *Other != Pair.Value)
				{
					bUpdate = true;
					break;
				}
			}
		}

		if(bUpdate)
		{
			ChangeNumberToZippedBinaries = NewChangeNumberToZippedBinaries;
			if(OnUpdateMetadata && Changes.Num() > 0)
			{
				OnUpdateMetadata();
			}
		}
	}

	return true;
}


} // namespace UGSCore
