// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Timespan.h"
#include "HAL/Runnable.h"
#include "HAL/Event.h"
#include "Perforce.h"
#include "OutputAdapters.h"

namespace UGSCore
{

struct FChangeType
{
	bool bContainsCode = false;
	bool bContainsContent = false;
};

class FPerforceMonitor : FRunnable
{
public:
	typedef TSharedRef<FPerforceChangeSummary, ESPMode::ThreadSafe> FChangeSharedRef;

	TFunction<void()> OnUpdate;
	TFunction<void()> OnUpdateMetadata;
	TFunction<void()> OnChangeTypeQueryFinished;
	TFunction<void()> OnStreamChange;

	FPerforceMonitor(const TSharedRef<FPerforceConnection>& InPerforce, const FString& InBranchClientPath, const FString& InSelectedClientFileName, const FString& InSelectedProjectIdentifier, const FString& InLogPath);
	~FPerforceMonitor();

	void Start();

	FString GetLastStatusMessage() const;
	int GetCurrentMaxChanges() const;
	int GetPendingMaxChanges() const;
	TArray<FString> GetOtherStreamNames() const;
	int GetLastChangeByCurrentUser() const;
	int GetLastCodeChangeByCurrentUser() const;
	bool HasZippedBinaries() const;

	TArray<FChangeSharedRef> GetChanges() const;
	bool TryGetChangeType(int ChangeNumber, FChangeType& OutType) const;
	bool TryGetArchivePathForChangeNumber(int ChangeNumber, FString& OutArchivePath) const;
	TSet<int> GetPromotedChangeNumbers() const;

	void Refresh();

private:
	TSharedRef<FPerforceConnection> Perforce;
	const FString BranchClientPath;
	const FString SelectedClientFileName;
	const FString SelectedProjectIdentifier;
	FBoundedLogWriter LogWriter;
	FRunnableThread* WorkerThread;
	TArray<FChangeSharedRef> Changes; // Sorted
	TMap<int, FChangeType> ChangeNumberToType;
	TSet<int> PromotedChangeNumbers;
	int ZippedBinariesConfigChangeNumber;
	FString ZippedBinariesPath;
	TMap<int, FString> ChangeNumberToZippedBinaries;
	FEvent* RefreshEvent;
	FEvent* AbortEvent;
	FString LastStatusMessage;
	int CurrentMaxChanges;
	int PendingMaxChanges;
	int LastChangeByCurrentUser;
	int LastCodeChangeByCurrentUser;
	TArray<FString> OtherStreamNames;
	bool bDisposing;
	mutable FCriticalSection CriticalSection;

	virtual uint32 Run() override;
	void RunInternal();

	bool UpdateChanges();
	bool UpdateChangeTypes();
	bool UpdateZippedBinaries();
};

} // namespace UGSCore
