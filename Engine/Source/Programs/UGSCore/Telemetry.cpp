// Copyright Epic Games, Inc. All Rights Reserved.

#include "Telemetry.h"

namespace UGSCore
{

//// FTelemetryStopwatch ////

FTelemetryStopwatch::FTelemetryStopwatch(const FString& InAction, const FString& InProject)
	: Action(InAction)
	, Project(InProject)
	, StartTime(FDateTime::UtcNow())
	, EndTime(FDateTime(0))
{
}

FTelemetryStopwatch::~FTelemetryStopwatch()
{
	if(Result.Len() == 0)
	{
		Stop(TEXT("Aborted"));
	}
//	TelemetryWriter.Enqueue(Action, Result, Project, StartTime, (float)Elapsed.TotalSeconds);
}

FTimespan FTelemetryStopwatch::Stop(const FString& InResult)
{
	EndTime = FDateTime::UtcNow();
	Result = InResult;
	return EndTime - StartTime;
}

FTimespan FTelemetryStopwatch::GetElapsed() const
{
	if(Result.Len() == 0)
	{
		return FDateTime::UtcNow() - StartTime;
	}
	else
	{
		return EndTime - StartTime;
	}
}

//// FTelemetryWriter ////

FTelemetryWriter::FTelemetryWriter(const FString& InSqlConnectionString, const FString& InLogFileName)
{
}

FTelemetryWriter::~FTelemetryWriter()
{
}

} // namespace UGSCore
