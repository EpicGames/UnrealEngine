// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Timespan.h"

namespace UGSCore
{

class FTelemetryStopwatch
{
public:
	FTelemetryStopwatch(const FString& InAction, const FString& InProject);
	~FTelemetryStopwatch();

	FTimespan Stop(const FString& InResult);
	FTimespan GetElapsed() const;

private:
	const FString Action;
	const FString Project;
	const FDateTime StartTime;
	FString Result;
	FDateTime EndTime;
};

class FTelemetryWriter
{
public:
	FTelemetryWriter(const FString& InSqlConnectionString, const FString& InLogFileName);
	~FTelemetryWriter();
};

} // namespace UGSCore
