// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UGSCore : ModuleRules
{
	public UGSCore(ReadOnlyTargetRules Target) : base(Target)
	{
		// UGS Shared library public includes
		PublicIncludePaths.Add("Programs/UGSCore/");

		PublicDependencyModuleNames.Add("Core");

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Json"
			}
		);

		PrivateIncludePaths.AddRange(
			new string[]
			{
			});

		bEnableExceptions = false;
		bUseRTTI = false;
	}
}
