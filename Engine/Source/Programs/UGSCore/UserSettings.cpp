// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserSettings.h"
#include "HAL/FileManager.h"
#include "Algo/Transform.h"
#include "Utility.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

namespace UGSCore
{

//// EBuildConfig ////

FString ToString(EBuildConfig BuildConfig)
{
	switch(BuildConfig)
	{
	case EBuildConfig::Debug:
		return TEXT("Debug");
	case EBuildConfig::DebugGame:
		return TEXT("DebugGame");
	case EBuildConfig::Development:
		return TEXT("Development");
	default:
		check(false);
		return FString();
	}
}

bool TryParse(const TCHAR* Text, EBuildConfig& OutBuildConfig)
{
	if (FCString::Stricmp(Text, TEXT("Debug")) == 0)
	{
		OutBuildConfig = EBuildConfig::Debug;
		return true;
	}
	else if (FCString::Stricmp(Text, TEXT("DebugGame")) == 0)
	{
		OutBuildConfig = EBuildConfig::DebugGame;
		return true;
	}
	else if (FCString::Stricmp(Text, TEXT("Development")) == 0)
	{
		OutBuildConfig = EBuildConfig::Development;
		return true;
	}
	return false;
}

//// ETabLabels ////

FString ToString(ETabLabels TabLabels)
{
	switch(TabLabels)
	{
	case ETabLabels::Stream:
		return TEXT("Stream");
	case ETabLabels::WorkspaceName:
		return TEXT("WorkspaceName");
	case ETabLabels::WorkspaceRoot:
		return TEXT("WorkspaceRoot");
	case ETabLabels::ProjectFile:
		return TEXT("ProjectFile");
	default:
		check(false);
		return FString();
	}
}

bool TryParse(const TCHAR* Text, ETabLabels& OutTabLabels)
{
	if (FCString::Stricmp(Text, TEXT("Stream")) == 0)
	{
		OutTabLabels = ETabLabels::Stream;
		return true;
	}
	else if (FCString::Stricmp(Text, TEXT("WorkspaceName")) == 0)
	{
		OutTabLabels = ETabLabels::WorkspaceName;
		return true;
	}
	else if (FCString::Stricmp(Text, TEXT("WorkspaceRoot")) == 0)
	{
		OutTabLabels = ETabLabels::WorkspaceRoot;
		return true;
	}
	else if (FCString::Stricmp(Text, TEXT("ProjectFile")) == 0)
	{
		OutTabLabels = ETabLabels::ProjectFile;
		return true;
	}
	return false;
}

//// ELatestChangeType ////

FString ToString(ELatestChangeType LatestChangeType)
{
	switch(LatestChangeType)
	{
	case ELatestChangeType::Any:
		return TEXT("Any");
	case ELatestChangeType::Good:
		return TEXT("Good");
	case ELatestChangeType::Starred:
		return TEXT("Starred");
	default:
		check(false);
		return FString();
	}
}

bool TryParse(const TCHAR* Text, ELatestChangeType& OutLatestChangeType)
{
	if(FCString::Stricmp(Text, TEXT("Any")) == 0)
	{
		OutLatestChangeType = ELatestChangeType::Any;
		return true;
	}
	else if(FCString::Stricmp(Text, TEXT("Good")) == 0)
	{
		OutLatestChangeType = ELatestChangeType::Good;
		return true;
	}
	else if(FCString::Stricmp(Text, TEXT("Starred")) == 0)
	{
		OutLatestChangeType = ELatestChangeType::Starred;
		return true;
	}
	return false;
}

//// FUserSettings ////

FUserSettings::FUserSettings(const FString& InFileName)
	: FileName(InFileName)
{
	if(IFileManager::Get().FileExists(*FileName))
	{
		ConfigFile.Load(*FileName);
	}

	// General settings
	bBuildAfterSync = (ConfigFile.GetValue(TEXT("General.BuildAfterSync"), 1) != 0);
	bRunAfterSync = (ConfigFile.GetValue(TEXT("General.RunAfterSync"), 1) != 0);
	bSyncPrecompiledEditor = (ConfigFile.GetValue(TEXT("General.SyncPrecompiledEditor"), 0) != 0);
	bOpenSolutionAfterSync = (ConfigFile.GetValue(TEXT("General.OpenSolutionAfterSync"), 0) != 0);
	bShowLogWindow = (ConfigFile.GetValue(TEXT("General.ShowLogWindow"), false));
	bAutoResolveConflicts = (ConfigFile.GetValue(TEXT("General.AutoResolveConflicts"), 1) != 0);
	bUseIncrementalBuilds = ConfigFile.GetValue(TEXT("General.IncrementalBuilds"), true);
	bShowLocalTimes = ConfigFile.GetValue(TEXT("General.ShowLocalTimes"), false);
	bShowAllStreams = ConfigFile.GetValue(TEXT("General.ShowAllStreams"), false);
	bKeepInTray = ConfigFile.GetValue(TEXT("General.KeepInTray"), true);
	FilterIndex = ConfigFile.GetValue(TEXT("General.FilterIndex"), 0);
	LastProjectFileName = ConfigFile.GetValue(TEXT("General.LastProjectFileName"), TEXT(""));

	ConfigFile.TryGetValues(TEXT("General.OpenProjectFileNames"), OpenProjectFileNames);
	if(LastProjectFileName.Len() > 0)
	{
		for(int Idx = 0;;Idx++)
		{
			if(Idx == OpenProjectFileNames.Num())
			{
				OpenProjectFileNames.Add(LastProjectFileName);
				break;
			}
			else if(OpenProjectFileNames[Idx] == LastProjectFileName)
			{
				break;
			}
		}
	}

	ConfigFile.TryGetValues(TEXT("General.OtherProjectFileNames"), OtherProjectFileNames);
	ConfigFile.TryGetValues(TEXT("General.SyncFilter"), SyncView);
	ConfigFile.TryGetValues(TEXT("General.SyncIncludedCategories"), SyncIncludedCategories);
	ConfigFile.TryGetValues(TEXT("General.SyncExcludedCategories"), SyncExcludedCategories);

	// TODO i set this default value as we assert when saving if no value is set but honestly idk what Any means
	SyncType = ELatestChangeType::Any;
	if(TryParse(ConfigFile.GetValue(TEXT("General.SyncType"), TEXT("")), SyncType))
	{
		SyncType = ELatestChangeType::Good;
	}

	// Build configuration
	const TCHAR* CompiledEditorBuildConfigName = ConfigFile.GetValue(TEXT("General.BuildConfig"), TEXT(""));
	if(!TryParse(CompiledEditorBuildConfigName, CompiledEditorBuildConfig))
	{
		CompiledEditorBuildConfig = EBuildConfig::DebugGame;
	}

	// Tab names
	const TCHAR* TabNamesValue = ConfigFile.GetValue(TEXT("General.TabNames"), TEXT(""));
	if(!TryParse(TabNamesValue, TabLabels))
	{
		TabLabels = ETabLabels::ProjectFile;
	}

	// Editor arguments
	TArray<FString> Arguments;
	if(!ConfigFile.TryGetValues(TEXT("General.EditorArguments"), Arguments))
	{
		Arguments.Add(TEXT("0:-log"));
		Arguments.Add(TEXT("0:-fastload"));
	}
	for(const FString& Argument : Arguments)
	{
		if(Argument.StartsWith(TEXT("0:")))
		{
			EditorArguments.Add(TTuple<FString,bool>(Argument.Mid(2), false));
		}
		else if(Argument.StartsWith(TEXT("1:")))
		{
			EditorArguments.Add(TTuple<FString,bool>(Argument.Mid(2), true));
		}
		else
		{
			EditorArguments.Add(TTuple<FString,bool>(Argument, true));
		}
	}

	// Window settings
	TSharedPtr<const FCustomConfigSection> WindowSection = ConfigFile.FindSection(TEXT("Window"));
	if(WindowSection.IsValid())
	{
		bHasWindowSettings = true;

		int X = WindowSection->GetValue(TEXT("X"), -1);
		int Y = WindowSection->GetValue(TEXT("Y"), -1);
		int Width = WindowSection->GetValue(TEXT("Width"), -1);
		int Height = WindowSection->GetValue(TEXT("Height"), -1);
//		WindowRectangle = new Rectangle(X, Y, Width, Height);

		FCustomConfigObject ColumnWidthObject(WindowSection->GetValue(TEXT("ColumnWidths"), TEXT("")));
		for(const TTuple<FString, FString>& ColumnWidthPair : ColumnWidthObject.Pairs)
		{
//			int Value;
//			if(int.TryParse(ColumnWidthPair.Value, out Value))
//			{
//				ColumnWidths[ColumnWidthPair.Key] = Value;
//			}
		}

		bWindowVisible = WindowSection->GetValue(TEXT("Visible"), true);
	}

	// Schedule settings
	bScheduleEnabled = ConfigFile.GetValue(TEXT("Schedule.Enabled"), false);
	if(!FTimespan::Parse(ConfigFile.GetValue(TEXT("Schedule.Time"), TEXT("")), ScheduleTime))
	{
		ScheduleTime = FTimespan(6, 0, 0);
	}
	if(!TryParse(ConfigFile.GetValue(TEXT("Schedule.Change"), TEXT("")), ScheduleChange))
	{
		ScheduleChange = ELatestChangeType::Good;
	}

	// Perforce settings
	SyncOptions.NumRetries = ConfigFile.GetValue(TEXT("Perforce.NumRetries"), 0);
	SyncOptions.NumThreads = ConfigFile.GetValue(TEXT("Perforce.NumThreads"), 0);
	SyncOptions.TcpBufferSize = ConfigFile.GetValue(TEXT("Perforce.TcpBufferSize"), 0);

	LoadGlobalSettings();
}

FString Trim(const FString& Text, const TCHAR Character)
{
	FString Result = Text;
	while(Result.StartsWith(TEXT("/")))
	{
		Result = Result.Mid(1);
	}
	while(Result.EndsWith(TEXT("/")))
	{
		Result = Result.Mid(0, Result.Len() - 1);
	}
	return Result;
}

TSharedRef<FUserWorkspaceSettings> FUserSettings::FindOrAddWorkspace(const TCHAR* ClientBranchPath)
{
	// Update the current workspace
	FString CurrentWorkspaceKey = Trim(ClientBranchPath, TEXT('/'));
	// TODO: Trim('/')?

	TSharedRef<FUserWorkspaceSettings>* CurrentWorkspaceEntry = WorkspaceKeyToSettings.Find(CurrentWorkspaceKey);
	if(CurrentWorkspaceEntry == nullptr)
	{
		// Create a new workspace settings object
		TSharedRef<FUserWorkspaceSettings> CurrentWorkspace = MakeShared<FUserWorkspaceSettings>();
		CurrentWorkspaceEntry = &(WorkspaceKeyToSettings.Add(CurrentWorkspaceKey, CurrentWorkspace));

		// Read the workspace settings
		TSharedPtr<FCustomConfigSection> WorkspaceSection = ConfigFile.FindSection(*CurrentWorkspaceKey);
		if(!WorkspaceSection.IsValid())
		{
			FString LegacyBranchAndClientKey = Trim(ClientBranchPath, TEXT('/'));

			int SlashIdx;
			if(LegacyBranchAndClientKey.FindChar(TEXT('/'), SlashIdx))
			{
				LegacyBranchAndClientKey = LegacyBranchAndClientKey.Mid(0, SlashIdx) + "$" + LegacyBranchAndClientKey.Mid(SlashIdx + 1);
			}

			FString CurrentSync;
			if(ConfigFile.TryGetValue(*(FString(TEXT("Clients.")) + LegacyBranchAndClientKey), CurrentSync))
			{
				int AtIdx;
				if(CurrentSync.FindLastChar(TEXT('@'), AtIdx))
				{
					int ChangeNumber;
					if(FUtility::TryParse(*CurrentSync.Mid(AtIdx + 1), ChangeNumber))
					{
						CurrentWorkspace->CurrentProjectIdentifier = CurrentSync.Mid(0, AtIdx);
						CurrentWorkspace->CurrentChangeNumber = ChangeNumber;
					}
				}
			}

			FString LastUpdateResultText;
			if(ConfigFile.TryGetValue(*(FString(TEXT("Clients.")) + LegacyBranchAndClientKey + TEXT("$LastUpdate")), LastUpdateResultText))
			{
				int ColonIdx;
				if(LastUpdateResultText.FindLastChar(TEXT(':'), ColonIdx))
				{
					int ChangeNumber;
					if(FUtility::TryParse(*LastUpdateResultText.Mid(0, ColonIdx), ChangeNumber))
					{
						EWorkspaceUpdateResult Result;
						if(TryParse(*LastUpdateResultText.Mid(ColonIdx + 1), Result))
						{
							CurrentWorkspace->LastSyncChangeNumber = ChangeNumber;
							CurrentWorkspace->LastSyncResult = Result;
						}
					}
				}
			}

			CurrentWorkspace->SyncView.Empty();
			CurrentWorkspace->SyncIncludedCategories.Empty();
			CurrentWorkspace->SyncExcludedCategories.Empty();
		}
		else
		{
			CurrentWorkspace->CurrentProjectIdentifier = WorkspaceSection->GetValue(TEXT("CurrentProjectPath"), TEXT(""));
			CurrentWorkspace->CurrentChangeNumber = WorkspaceSection->GetValue(TEXT("CurrentChangeNumber"), -1);
			WorkspaceSection->TryGetValues(TEXT("AdditionalChangeNumbers"), CurrentWorkspace->AdditionalChangeNumbers);
			if(!TryParse(WorkspaceSection->GetValue(TEXT("LastSyncResult"), TEXT("")), CurrentWorkspace->LastSyncResult))
			{
				CurrentWorkspace->LastSyncResult = EWorkspaceUpdateResult::Canceled;
			}
			CurrentWorkspace->LastSyncResultMessage = UnescapeText(WorkspaceSection->GetValue(TEXT("LastSyncResultMessage"), TEXT("")));
			CurrentWorkspace->LastSyncChangeNumber = WorkspaceSection->GetValue(TEXT("LastSyncChangeNumber"), -1);

			FDateTime LastSyncTime;
			if(FDateTime::Parse(WorkspaceSection->GetValue(TEXT("LastSyncTime"), TEXT("")), LastSyncTime))
			{
				CurrentWorkspace->LastSyncTime = LastSyncTime;
			}

			CurrentWorkspace->LastSyncDurationSeconds = WorkspaceSection->GetValue(TEXT("LastSyncDuration"), 0);
			CurrentWorkspace->LastBuiltChangeNumber = WorkspaceSection->GetValue(TEXT("LastBuiltChangeNumber"), 0);
			WorkspaceSection->TryGetValues(TEXT("ExpandedArchiveName"), CurrentWorkspace->ExpandedArchiveTypes);

			WorkspaceSection->TryGetValues(TEXT("SyncFilter"), CurrentWorkspace->SyncView);
			WorkspaceSection->TryGetValues(TEXT("SyncIncludedCategories"), CurrentWorkspace->SyncIncludedCategories);
			WorkspaceSection->TryGetValues(TEXT("SyncExcludedCategories"), CurrentWorkspace->SyncExcludedCategories);
		}
	}
	return *CurrentWorkspaceEntry;
}

TSharedRef<FUserProjectSettings> FUserSettings::FindOrAddProject(const TCHAR* ClientProjectFileName)
{
	// Read the project settings
	TSharedRef<FUserProjectSettings>* CurrentProject = ProjectKeyToSettings.Find(ClientProjectFileName);
	if(CurrentProject == nullptr)
	{
		TSharedRef<FUserProjectSettings> NewCurrentProject = MakeShared<FUserProjectSettings>();
		CurrentProject = &(ProjectKeyToSettings.Add(ClientProjectFileName, NewCurrentProject));
	
		TArray<FString> BuildStepStrings;
		TSharedRef<FCustomConfigSection> ProjectSection = ConfigFile.FindOrAddSection(ClientProjectFileName);
		ProjectSection->TryGetValues(TEXT("BuildStep"), BuildStepStrings);
		Algo::Transform(BuildStepStrings, NewCurrentProject->BuildSteps, [](const FString& Text) -> FCustomConfigObject { return FCustomConfigObject(*Text); });
	}
	return *CurrentProject;
}

void FUserSettings::Save()
{
	// General settings
	TSharedRef<FCustomConfigSection> GeneralSection = ConfigFile.FindOrAddSection(TEXT("General"));
	GeneralSection->Clear();
	GeneralSection->SetValue(TEXT("BuildAfterSync"), bBuildAfterSync);
	GeneralSection->SetValue(TEXT("RunAfterSync"), bRunAfterSync);
	GeneralSection->SetValue(TEXT("SyncPrecompiledEditor"), bSyncPrecompiledEditor);
	GeneralSection->SetValue(TEXT("OpenSolutionAfterSync"), bOpenSolutionAfterSync);
	GeneralSection->SetValue(TEXT("ShowLogWindow"), bShowLogWindow);
	GeneralSection->SetValue(TEXT("AutoResolveConflicts"), bAutoResolveConflicts);
	GeneralSection->SetValue(TEXT("IncrementalBuilds"), bUseIncrementalBuilds);
	GeneralSection->SetValue(TEXT("ShowLocalTimes"), bShowLocalTimes);
	GeneralSection->SetValue(TEXT("ShowAllStreams"), bShowAllStreams);
	GeneralSection->SetValue(TEXT("LastProjectFileName"), *LastProjectFileName);
	GeneralSection->SetValues(TEXT("OpenProjectFileNames"), OpenProjectFileNames);
	GeneralSection->SetValue(TEXT("KeepInTray"), bKeepInTray);
	GeneralSection->SetValue(TEXT("FilterIndex"), FilterIndex);
	GeneralSection->SetValues(TEXT("OtherProjectFileNames"), OtherProjectFileNames);
	GeneralSection->SetValues(TEXT("SyncFilter"), SyncView);
	GeneralSection->SetValues(TEXT("SyncIncludedCategories"), SyncIncludedCategories);
	GeneralSection->SetValues(TEXT("SyncExcludedCategories"), SyncExcludedCategories);
	GeneralSection->SetValue(TEXT("SyncType"), *ToString(SyncType));

	// Build configuration
	GeneralSection->SetValue(TEXT("BuildConfig"), *ToString(CompiledEditorBuildConfig));

	// Tab names
	GeneralSection->SetValue(TEXT("TabNames"), *ToString(TabLabels));

	// Editor arguments
	TArray<FString> EditorArgumentList;
	for(const TTuple<FString, bool>& EditorArgument : EditorArguments)
	{
		EditorArgumentList.Add(FString::Printf(TEXT("%d:%s"), EditorArgument.Value? 1 : 0, *EditorArgument.Key));
	}
	GeneralSection->SetValues(TEXT("EditorArguments"), EditorArgumentList);

	// Schedule settings
	TSharedRef<FCustomConfigSection> ScheduleSection = ConfigFile.FindOrAddSection(TEXT("Schedule"));
	ScheduleSection->Clear();
	ScheduleSection->SetValue(TEXT("Enabled"), bScheduleEnabled);
	ScheduleSection->SetValue(TEXT("Time"), *ScheduleTime.ToString());
	ScheduleSection->SetValue(TEXT("Change"), *ToString(ScheduleChange));

	// Window settings
	if(bHasWindowSettings)
	{
		TSharedRef<FCustomConfigSection> WindowSection = ConfigFile.FindOrAddSection(TEXT("Window"));
		WindowSection->Clear();
//		WindowSection.SetValue(TEXT("X"), WindowRectangle.X);
//		WindowSection.SetValue(TEXT("Y"), WindowRectangle.Y);
//		WindowSection.SetValue(TEXT("Width"), WindowRectangle.Width);
//		WindowSection.SetValue(TEXT("Height"), WindowRectangle.Height);

		FCustomConfigObject ColumnWidthsObject;
		for(const TTuple<FString, int>& ColumnWidthPair : ColumnWidths)
		{
			ColumnWidthsObject.SetValue(*ColumnWidthPair.Key, ColumnWidthPair.Value);
		}
		WindowSection->SetValue(TEXT("ColumnWidths"), *ColumnWidthsObject.ToString());

		WindowSection->SetValue(TEXT("Visible"), bWindowVisible);
	}

	// Current workspace settings
	for(TTuple<FString, TSharedRef<FUserWorkspaceSettings>>& Pair : WorkspaceKeyToSettings)
	{
		const FString& CurrentWorkspaceKey = Pair.Key;
		FUserWorkspaceSettings& CurrentWorkspace = Pair.Value.Get();

		TSharedRef<FCustomConfigSection> WorkspaceSection = ConfigFile.FindOrAddSection(*CurrentWorkspaceKey);
		WorkspaceSection->Clear();
		WorkspaceSection->SetValue(TEXT("CurrentProjectPath"), *CurrentWorkspace.CurrentProjectIdentifier);
		WorkspaceSection->SetValue(TEXT("CurrentChangeNumber"), CurrentWorkspace.CurrentChangeNumber);
		WorkspaceSection->SetValues(TEXT("AdditionalChangeNumbers"), CurrentWorkspace.AdditionalChangeNumbers);
		WorkspaceSection->SetValue(TEXT("LastSyncResult"), *ToString(CurrentWorkspace.LastSyncResult));
		WorkspaceSection->SetValue(TEXT("LastSyncResultMessage"), *EscapeText(CurrentWorkspace.LastSyncResultMessage));
		WorkspaceSection->SetValue(TEXT("LastSyncChangeNumber"), CurrentWorkspace.LastSyncChangeNumber);
		if(CurrentWorkspace.LastSyncTime.IsSet())
		{
			WorkspaceSection->SetValue(TEXT("LastSyncTime"), *CurrentWorkspace.LastSyncTime.GetValue().ToString());
		}
		if(CurrentWorkspace.LastSyncDurationSeconds > 0)
		{
			WorkspaceSection->SetValue(TEXT("LastSyncDuration"), CurrentWorkspace.LastSyncDurationSeconds);
		}
		WorkspaceSection->SetValue(TEXT("LastBuiltChangeNumber"), CurrentWorkspace.LastBuiltChangeNumber);
		WorkspaceSection->SetValues(TEXT("ExpandedArchiveName"), CurrentWorkspace.ExpandedArchiveTypes);
		WorkspaceSection->SetValues(TEXT("SyncFilter"), CurrentWorkspace.SyncView);
		WorkspaceSection->SetValues(TEXT("SyncIncludedCategories"), CurrentWorkspace.SyncIncludedCategories);
		WorkspaceSection->SetValues(TEXT("SyncExcludedCategories"), CurrentWorkspace.SyncExcludedCategories);
	}

	// Current project settings
	for(TTuple<FString, TSharedRef<FUserProjectSettings>>& Pair : ProjectKeyToSettings)
	{
		const FString& CurrentProjectKey = Pair.Key;
		FUserProjectSettings& CurrentProject = Pair.Value.Get();

		TSharedRef<FCustomConfigSection> ProjectSection = ConfigFile.FindOrAddSection(*CurrentProjectKey);
		ProjectSection->Clear();

		TArray<FString> BuildStepStrings;
		Algo::Transform(CurrentProject.BuildSteps, BuildStepStrings, [](const FCustomConfigObject& Step){ return Step.ToString(); });
		ProjectSection->SetValues(TEXT("BuildStep"), BuildStepStrings);
	}

	// Perforce settings
	TSharedRef<FCustomConfigSection> PerforceSection = ConfigFile.FindOrAddSection(TEXT("Perforce"));
	PerforceSection->Clear();
	if(SyncOptions.NumRetries > 0)
	{
		PerforceSection->SetValue(TEXT("NumRetries"), SyncOptions.NumRetries);
	}
	if(SyncOptions.NumThreads > 0)
	{
		PerforceSection->SetValue(TEXT("NumThreads"), SyncOptions.NumThreads);
	}
	if(SyncOptions.TcpBufferSize > 0)
	{
		PerforceSection->SetValue(TEXT("TcpBufferSize"), SyncOptions.TcpBufferSize);
	}

	// Save the file
	ConfigFile.Save(*FileName);

	SaveGlobalSettings();
}

TArray<FString> FUserSettings::GetCombinedSyncFilter(const TMap<FGuid, FWorkspaceSyncCategory>& UniqueIdToFilter, const TArray<FString>& GlobalView, const TArray<FGuid>& GlobalExcludedCategories, const TArray<FString>& WorkspaceView, const TArray<FGuid>& WorkspaceExcludedCategories)
{
	TArray<FString> Lines;

	// Add the custom global view
	for(const FString& GlobalViewLine : GlobalView)
	{
		FString TrimLine = GlobalViewLine.TrimStartAndEnd();
		if(TrimLine.Len() > 0 && !TrimLine.StartsWith(TEXT(";")))
		{
			Lines.Add(MoveTemp(TrimLine));
		}
	}

	// Add the custom workspace view
	for(const FString& WorkspaceViewLine : GlobalView)
	{
		FString TrimLine = WorkspaceViewLine.TrimStartAndEnd();
		if(TrimLine.Len() > 0 && !TrimLine.StartsWith(TEXT(";")))
		{
			Lines.Add(MoveTemp(TrimLine));
		}
	}

	// Add all the excluded categories
	for(const TTuple<FGuid, FWorkspaceSyncCategory>& Pair : UniqueIdToFilter)
	{
		if(Pair.Value.bEnable && (GlobalExcludedCategories.Contains(Pair.Key) || WorkspaceExcludedCategories.Contains(Pair.Key)))
		{
			for(const FString& Path : Pair.Value.Paths)
			{
				Lines.Add(FString::Printf(TEXT("-%s"), *Path));
			}
		}
	}
	return Lines;
}

static FString GetGlobalJsonPath()
{
	FPlatformProcess::ApplicationSettingsContext Context;
	Context.Location = FPlatformProcess::ApplicationSettingsContext::Context::LocalUser;
	Context.bIsEpic = false;
	return FPlatformProcess::GetApplicationSettingsDir(Context) / TEXT("UnrealGameSync") / TEXT("Global.json");
}

void FUserSettings::LoadGlobalSettings()
{
	FString GlobalJsonPath = GetGlobalJsonPath();

	FString JsonString;
	if (!FFileHelper::LoadFileToString(JsonString, *GlobalJsonPath))
	{
		return;
	}

	TSharedPtr<FJsonObject> RootObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);
	if (!FJsonSerializer::Deserialize(Reader, RootObject) || !RootObject.IsValid())
	{
		return;
	}

	const TSharedPtr<FJsonObject>* FilterObject;
	if (!RootObject->TryGetObjectField(TEXT("Filter"), FilterObject))
	{
		return;
	}

	const TArray<TSharedPtr<FJsonValue>>* ExcludeArray;
	if ((*FilterObject)->TryGetArrayField(TEXT("ExcludeCategories"), ExcludeArray))
	{
		SyncExcludedCategories.Empty();
		for (const TSharedPtr<FJsonValue>& Value : *ExcludeArray)
		{
			FGuid Guid;
			if (FGuid::Parse(Value->AsString(), Guid))
			{
				SyncExcludedCategories.Add(Guid);
			}
		}
	}

	const TArray<TSharedPtr<FJsonValue>>* IncludeArray;
	if ((*FilterObject)->TryGetArrayField(TEXT("IncludeCategories"), IncludeArray))
	{
		SyncIncludedCategories.Empty();
		for (const TSharedPtr<FJsonValue>& Value : *IncludeArray)
		{
			FGuid Guid;
			if (FGuid::Parse(Value->AsString(), Guid))
			{
				SyncIncludedCategories.Add(Guid);
			}
		}
	}

	const TArray<TSharedPtr<FJsonValue>>* ViewArray;
	if ((*FilterObject)->TryGetArrayField(TEXT("View"), ViewArray))
	{
		SyncView.Empty();
		for (const TSharedPtr<FJsonValue>& Value : *ViewArray)
		{
			SyncView.Add(Value->AsString());
		}
	}
}

void FUserSettings::SaveGlobalSettings()
{
	FString GlobalJsonPath = GetGlobalJsonPath();

	// Read-modify-write to preserve other fields in Global.json
	TSharedPtr<FJsonObject> RootObject;
	FString JsonString;
	if (FFileHelper::LoadFileToString(JsonString, *GlobalJsonPath))
	{
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);
		FJsonSerializer::Deserialize(Reader, RootObject);
	}
	if (!RootObject.IsValid())
	{
		RootObject = MakeShared<FJsonObject>();
	}

	TSharedPtr<FJsonObject> FilterObject = MakeShared<FJsonObject>();

	TArray<TSharedPtr<FJsonValue>> IncludeValues;
	for (const FGuid& Guid : SyncIncludedCategories)
	{
		IncludeValues.Add(MakeShared<FJsonValueString>(Guid.ToString(EGuidFormats::DigitsWithHyphensLower)));
	}
	FilterObject->SetArrayField(TEXT("IncludeCategories"), IncludeValues);

	TArray<TSharedPtr<FJsonValue>> ExcludeValues;
	for (const FGuid& Guid : SyncExcludedCategories)
	{
		ExcludeValues.Add(MakeShared<FJsonValueString>(Guid.ToString(EGuidFormats::DigitsWithHyphensLower)));
	}
	FilterObject->SetArrayField(TEXT("ExcludeCategories"), ExcludeValues);

	TArray<TSharedPtr<FJsonValue>> ViewValues;
	for (const FString& ViewLine : SyncView)
	{
		ViewValues.Add(MakeShared<FJsonValueString>(ViewLine));
	}
	FilterObject->SetArrayField(TEXT("View"), ViewValues);

	RootObject->SetObjectField(TEXT("Filter"), FilterObject);

	FString OutputString;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
	FJsonSerializer::Serialize(RootObject.ToSharedRef(), Writer);
	FFileHelper::SaveStringToFile(OutputString, *GlobalJsonPath);
}

FString FUserSettings::EscapeText(const FString& Text)
{
	FString Result;
	for(int Idx = 0; Idx < Text.Len(); Idx++)
	{
		switch(Text[Idx])
		{
			case '\\':
				Result += TEXT("\\\\");
				break;
			case '\t':
				Result += TEXT("\\t");
				break;
			case '\r':
				Result += TEXT("\\r");
				break;
			case '\n':
				Result += TEXT("\\n");
				break;
			case '\'':
				Result += TEXT("\\\'");
				break;
			case '\"':
				Result += TEXT("\\\"");
				break;
			default:
				Result += Text[Idx];
				break;
		}
	}
	return Result;
}

FString FUserSettings::UnescapeText(const FString& Text)
{
	FString Result;
	for(int Idx = 0; Idx < Text.Len(); Idx++)
	{
		if(Text[Idx] == '\\' && Idx + 1 < Text.Len())
		{
			switch(Text[++Idx])
			{
				case 't':
					Result += TEXT('\t');
					break;
				case 'r':
					Result += TEXT('\r');
					break;
				case 'n':
					Result += TEXT('\n');
					break;
				case '\'':
					Result += TEXT('\'');
					break;
				case '\"':
					Result += TEXT('\"');
					break;
				default:
					Result += Text[Idx];
					break;
			}
		}
		else
		{
			Result += Text[Idx];
		}
	}
	return Result;
}

} // namespace UGSCore
