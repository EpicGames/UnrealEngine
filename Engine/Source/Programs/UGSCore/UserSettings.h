// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CustomConfigFile.h"
#include "Perforce.h"
#include "Misc/Guid.h"
#include "Workspace.h"

namespace UGSCore
{

enum class EBuildConfig
{
	Debug,
	DebugGame,
	Development,
};

FString ToString(EBuildConfig BuildConfig);
bool TryParse(const TCHAR* Text, EBuildConfig& OutBuildConfig);

enum class ETabLabels
{
	Stream,
	WorkspaceName,
	WorkspaceRoot,
	ProjectFile,
};

FString ToString(ETabLabels Lables);
bool TryParse(const TCHAR* Text, ETabLabels& OutTabLabels);

enum class ELatestChangeType
{
	Any,
	Good,
	Starred,
};

FString ToString(ELatestChangeType LatestChangeType);
bool TryParse(const TCHAR* Text, ELatestChangeType& OutLatestChangeType);

struct FUserWorkspaceSettings
{
	// Settings for the currently synced project in this workspace. CurrentChangeNumber is only valid for this workspace if CurrentProjectPath is the current project.
	FString CurrentProjectIdentifier;
	int CurrentChangeNumber;
	TArray<int> AdditionalChangeNumbers;

	// Settings for the last attempted sync. These values are set to persist error messages between runs.
	int LastSyncChangeNumber;
	EWorkspaceUpdateResult LastSyncResult;
	FString LastSyncResultMessage;
	TOptional<FDateTime> LastSyncTime;
	int LastSyncDurationSeconds;

	// The last successful build, regardless of whether a failed sync has happened in the meantime. Used to determine whether to force a clean due to entries in the project config file.
	int LastBuiltChangeNumber;

	// Expanded archives in the workspace
	TArray<FString> ExpandedArchiveTypes;

	// Workspace specific SyncFilters
	TArray<FString> SyncView;
	TArray<FGuid> SyncIncludedCategories;
	TArray<FGuid> SyncExcludedCategories;
};

struct FUserProjectSettings
{
	TArray<FCustomConfigObject> BuildSteps;
};

struct FUserSettings
{
	const FString FileName;
	FCustomConfigFile ConfigFile;

	// General settings
	bool bBuildAfterSync;
	bool bRunAfterSync;
	bool bSyncPrecompiledEditor;
	bool bOpenSolutionAfterSync;
	bool bShowLogWindow;
	bool bAutoResolveConflicts;
	bool bUseIncrementalBuilds;
	bool bShowLocalTimes;
	bool bShowAllStreams;
	bool bKeepInTray;
	int FilterIndex;
	FString LastProjectFileName;
	TArray<FString> OpenProjectFileNames;
	TArray<FString> OtherProjectFileNames;
	TArray<FString> SyncView;
	TArray<FGuid> SyncIncludedCategories;
	TArray<FGuid> SyncExcludedCategories;
	ELatestChangeType SyncType;
	EBuildConfig CompiledEditorBuildConfig; // NB: This assumes not using precompiled editor. See CurrentBuildConfig.
	ETabLabels TabLabels;

	// Window settings
	bool bHasWindowSettings;
//	Rectangle WindowRectangle;
	TMap<FString, int> ColumnWidths;
	bool bWindowVisible;
		
	// Schedule settings
	bool bScheduleEnabled;
	FTimespan ScheduleTime;
	ELatestChangeType ScheduleChange;

	// Run configuration
	TArray<TTuple<FString, bool>> EditorArguments;

	// Project settings
	TMap<FString, TSharedRef<FUserWorkspaceSettings>> WorkspaceKeyToSettings;
	TMap<FString, TSharedRef<FUserProjectSettings>> ProjectKeyToSettings;

	// Perforce settings
	FPerforceSyncOptions SyncOptions;

	FUserSettings(const FString& InFileName);
	TSharedRef<FUserWorkspaceSettings> FindOrAddWorkspace(const TCHAR* ClientBranchPath);
	TSharedRef<FUserProjectSettings> FindOrAddProject(const TCHAR* ClientProjectFileName);
	void Save();

	static TArray<FString> GetCombinedSyncFilter(const TMap<FGuid, FWorkspaceSyncCategory>& UniqueIdToFilter, const TArray<FString>& GlobalView, const TArray<FGuid>& GlobalExcludedCategories, const TArray<FString>& WorkspaceView, const TArray<FGuid>& WorkspaceExcludedCategories);

	static FString EscapeText(const FString& Text);
	static FString UnescapeText(const FString& Text);

private:
	void LoadGlobalSettings();
	void SaveGlobalSettings();
};

} // namespace UGSCore
