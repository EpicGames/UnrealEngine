// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utility.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "HAL/Event.h"

DEFINE_LOG_CATEGORY(LogUGSCore);

namespace UGSCore
{

bool FUtility::TryParse(const TCHAR* Text, int32& OutValue)
{
	TCHAR* TextEnd;
	OutValue = FCString::Strtoi(Text, &TextEnd, 10);
	return (TextEnd != Text && *TextEnd == 0);
}

bool FUtility::TryParse(const TCHAR* Text, int64& OutValue)
{
	TCHAR* TextEnd;
	OutValue = FCString::Strtoi64(Text, &TextEnd, 10);
	return (TextEnd != Text && *TextEnd == 0);
}

bool FUtility::IsFileUnderDirectory(const TCHAR* FileName, const TCHAR* DirectoryName)
{
	FString FullDirectoryName = FPaths::ConvertRelativePathToFull(DirectoryName);
	FPaths::MakePlatformFilename(FullDirectoryName);

	if(!FullDirectoryName.EndsWith(FPlatformMisc::GetDefaultPathSeparator()))
	{
		FullDirectoryName += FPlatformMisc::GetDefaultPathSeparator();
	}

	FString FullFileName = FPaths::ConvertRelativePathToFull(FileName);
	FPaths::MakePlatformFilename(FullFileName);

	return FullFileName.StartsWith(FullDirectoryName);
}

FString FUtility::GetPathWithCorrectCase(const FString& Path)
{
	// Visitor which corrects the case of a filename against any matching item in a directory
	struct FFixCaseVisitor : public IPlatformFile::FDirectoryVisitor
	{
		FString Name;

		FFixCaseVisitor(FString InName) : Name(InName)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			if (Name == FilenameOrDirectory)
			{
				Name = FilenameOrDirectory;
				return false;
			}
			return true;
		}
	};

	FString ParentDir = FPaths::GetPath(Path);
	if(ParentDir.Len() == 0)
	{
		return Path;
	}
#if PLATFORM_WINDOWS
	else if(ParentDir.Len() == 2 && ParentDir[1] == ':')
	{
		ParentDir = ParentDir.ToUpper() + FPlatformMisc::GetDefaultPathSeparator();
	}
#endif
	else
	{
		ParentDir = GetPathWithCorrectCase(ParentDir) + FPlatformMisc::GetDefaultPathSeparator();
	}

	FFixCaseVisitor Visitor(ParentDir + FPaths::GetCleanFilename(Path));
	IFileManager::Get().IterateDirectory(*ParentDir, Visitor);
	return Visitor.Name;
}

FString FUtility::FormatUserName(const TCHAR* UserName)
{
	FString NormalUserName;
	for(int Idx = 0; UserName[Idx] != 0; Idx++)
	{
		if(Idx == 0 || UserName[Idx - 1] == '.')
		{
			NormalUserName += FChar::ToUpper(UserName[Idx]);
		}
		else if(UserName[Idx] == '.')
		{
			NormalUserName += ' ';
		}
		else
		{
			NormalUserName += UserName[Idx];
		}
	}
	return NormalUserName;
}

int FUtility::ExecuteProcess(const TCHAR* FileName, const TCHAR* CommandLine, const TCHAR* Input, FEvent* AbortEvent, FLineBasedTextWriter& Log)
{
	return ExecuteProcess(FileName, CommandLine, Input, AbortEvent, [&Log](const FString& Line){ Log.WriteLine(Line); });
}

int FUtility::ExecuteProcess(const TCHAR* FileName, const TCHAR* CommandLine, const TCHAR* Input, FEvent* AbortEvent, TArray<FString>& OutLines)
{
	uint32 ProcId;
	void* ReadPipe = nullptr;
	void* WritePipe = nullptr;
	FPlatformProcess::CreatePipe(ReadPipe, WritePipe);

	FProcHandle Proc = FPlatformProcess::CreateProc(FileName, CommandLine, false, true, true, &ProcId, 0, nullptr, WritePipe);

	FString Output;
	FString LatestOutput = FPlatformProcess::ReadPipe(ReadPipe);
	while (FPlatformProcess::IsProcRunning(Proc) || !LatestOutput.IsEmpty())
	{
		Output += LatestOutput;
		LatestOutput = FPlatformProcess::ReadPipe(ReadPipe);

		if (AbortEvent->Wait(FTimespan::Zero()))
		{
			FPlatformProcess::TerminateProc(Proc);
			FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
			FPlatformProcess::CloseProc(Proc);
			return -1;
		}
	}

	FPlatformProcess::ClosePipe(ReadPipe, WritePipe);

	// TODO worried this may not always work for say p4 servers sending out only \n on Windows host
	Output.ParseIntoArray(OutLines, LINE_TERMINATOR);

	int ExitCode = -1;
	bool GotReturnCode = FPlatformProcess::GetProcReturnCode(Proc, &ExitCode);

	FPlatformProcess::CloseProc(Proc);
	
	if (GotReturnCode)
	{
		return ExitCode;
	}

	return -1;
}

int FUtility::ExecuteProcess(const TCHAR* FileName, const TCHAR* CommandLine, const TCHAR* Input, FEvent* AbortEvent, const TFunction<void(const FString&)>& OutputLine)
{
	uint32 ProcId;
	void* ReadPipe = nullptr;
	void* WritePipe = nullptr;
	FPlatformProcess::CreatePipe(ReadPipe, WritePipe);

	FProcHandle Proc = FPlatformProcess::CreateProc(FileName, CommandLine, false, true, true, &ProcId, 0, nullptr, WritePipe);

	FString LatestOutput = FPlatformProcess::ReadPipe(ReadPipe);
	while (FPlatformProcess::IsProcRunning(Proc) || !LatestOutput.IsEmpty())
	{
		// Check if we end up having a perfect data blob, with a perfect amount of lines
		bool bEndsWithLineTerminator = LatestOutput.EndsWith(LINE_TERMINATOR);

		// Parsing into this array will give us full lines, minus the last entry will be the left over data
		TArray<FString> Lines;
		// TODO ... worried this LINE_TERMINATOR *may* not always be correct urg. For example p4 is on a Linux server, and on Windows im not sure if it will return \r\n
		LatestOutput.ParseIntoArray(Lines, LINE_TERMINATOR);

		// If there is no data, we cannot, if theres only 1 entry or we dont have any lines found in the data chunk, move all over to the LeftOverLine
		FString LeftOverLine;

		// If we have more then 2 lines, or we only have 1 line and it didnt contain *a* LINE_TERMINATOR its left over data
		if (Lines.Num() > 1 || (Lines.Num() > 0 && !bEndsWithLineTerminator))
		{
			LeftOverLine = Lines.Pop();
		}

		for (const FString& Line : Lines)
		{
			OutputLine(Line);
		}

		LatestOutput = LeftOverLine + FPlatformProcess::ReadPipe(ReadPipe);

		if (AbortEvent->Wait(FTimespan::Zero()))
		{
			FPlatformProcess::TerminateProc(Proc);
			FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
			FPlatformProcess::CloseProc(Proc);
			return -1;
		}
	}

	FPlatformProcess::ClosePipe(ReadPipe, WritePipe);

	int ExitCode = -1;
	bool GotReturnCode = FPlatformProcess::GetProcReturnCode(Proc, &ExitCode);

	FPlatformProcess::CloseProc(Proc);

	if (GotReturnCode)
	{
		return ExitCode;
	}

	return -1;
}

FString FUtility::ExpandVariables(const TCHAR* InputString, const TMap<FString, FString>* AdditionalVariables)
{
	FString Result = InputString;
	for(int Idx = 0;;)
	{
		Idx = Result.Find(TEXT("$("), ESearchCase::CaseSensitive, ESearchDir::FromStart, Idx);
		if(Idx == INDEX_NONE)
		{
			break;
		}
	
		// Find the end of the variable name
		int EndIdx = Result.Find(TEXT(")"), ESearchCase::CaseSensitive, ESearchDir::FromStart, Idx + 2);
		if (EndIdx == INDEX_NONE)
		{
			break;
		}

		// Extract the variable name from the string
		FString Name = Result.Mid(Idx + 2, EndIdx - (Idx + 2));

		// Find the value for it, either from the dictionary or the environment block
		const FString* VariableValue = nullptr;
		if(AdditionalVariables != nullptr)
		{
			VariableValue = AdditionalVariables->Find(Name);
		}

		FString Value;
		if(VariableValue != nullptr)
		{
			Value = *VariableValue;
		}
		else
		{
			Value = FPlatformMisc::GetEnvironmentVariable(*Name);
			if(Value.IsEmpty())
			{
				Idx = EndIdx + 1;
				continue;
			}
		}

		// Replace the variable, or skip past it
		Result = Result.Mid(0, Idx) + Value + Result.Mid(EndIdx + 1);
	}
	return Result;
}

} // namespace UGSCore
