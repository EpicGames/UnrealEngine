// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OutputAdapters.h"

DECLARE_LOG_CATEGORY_EXTERN(LogUGSCore, Log, All);

namespace UGSCore
{

struct FAbortException
{
};

struct FUtility
{
	static bool TryParse(const TCHAR* Text, int32& OutValue);
	static bool TryParse(const TCHAR* Text, int64& OutValue);

	static bool IsFileUnderDirectory(const TCHAR* FileName, const TCHAR* DirectoryName);
	static FString GetPathWithCorrectCase(const FString& Path);

	static FString FormatUserName(const TCHAR* UserName);

	static int ExecuteProcess(const TCHAR* FileName, const TCHAR* CommandLine, const TCHAR* Input, FEvent* AbortEvent, FLineBasedTextWriter& Log);
	static int ExecuteProcess(const TCHAR* FileName, const TCHAR* CommandLine, const TCHAR* Input, FEvent* AbortEvent, TArray<FString>& OutLines);
	static int ExecuteProcess(const TCHAR* FileName, const TCHAR* CommandLine, const TCHAR* Input, FEvent* AbortEvent, const TFunction<void(const FString&)>& OutputLine);

	/**
	 * Expands variables in $(VarName) format in the given string. Variables are retrieved from the given dictionary, or through the environment of the current process.
	 * Any unknown variables are ignored.
	 *
	 * @param InputString String to search for variable names
	 * @param Variables Lookup of variable names to values
	 * @return String with all variables replaced
	 */
	static FString ExpandVariables(const TCHAR* InputString, const TMap<FString, FString>* AdditionalVariables = nullptr);
};

} // namespace UGSCore
