// Copyright Epic Games, Inc. All Rights Reserved.

#include "Workspace.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "HAL/FileManager.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Event.h"
#include "Algo/AnyOf.h"
#include "Misc/FileHelper.h"
#include "BuildStep.h"
#include "Telemetry.h"
#include "Utility.h"

namespace UGSCore
{

const TCHAR* FWorkspace::DefaultBuildTargets[] =
{
	TEXT("UnrealHeaderTool Win64 Development, 0.1"),
	TEXT("$(EditorTarget) Win64 $(EditorConfiguration), 0.7"),
	TEXT("ShaderCompileWorker Win64 Development, 0.8"),
	TEXT("UnrealLightmass Win64 Development, 0.9"),
	TEXT("CrashReportClient Win64 Shipping, 1.0"),
	TEXT("InterchangeWorker Win64 Development, 1.0"),
};

const FWorkspaceSyncCategory FWorkspace::DefaultSyncCategories[] =
{
	FWorkspaceSyncCategory(FGuid(0x6703E989, 0xD912451D, 0x93ADB48D, 0xE748D282), TEXT("Content"), TEXT("*.uasset")),
	FWorkspaceSyncCategory(FGuid(0x6507C2FB, 0x19DD403A, 0xAFA3BBF8, 0x98248D5A), TEXT("Documentation"), TEXT("/Engine/Documentation/...")),
	FWorkspaceSyncCategory(FGuid(0xFD7C716E, 0x4BAD43AE, 0x8FAE8748, 0xEF9EE44D), TEXT("Platform Support: Android"), TEXT("/Engine/Source/ThirdParty/.../Android/...")),
	FWorkspaceSyncCategory(FGuid(0x3299A73D, 0x21764C0F, 0xBC99C1C6, 0x631AF6C4), TEXT("Platform Support: HTML5"), TEXT("/Engine/Source/ThirdParty/.../HTML5/...;/Engine/Extras/ThirdPartyNotUE/emsdk/...")),
	FWorkspaceSyncCategory(FGuid(0x176B2EB2, 0x35F74E8E, 0xB1315F1C, 0x5F0959AF), TEXT("Platform Support: iOS"), TEXT("/Engine/Source/ThirdParty/.../IOS/...")),
	FWorkspaceSyncCategory(FGuid(0xF44B2D25, 0xCBC04A8F, 0xB6B3E4A8, 0x125533DD), TEXT("Platform Support: Linux"), TEXT("/Engine/Source/ThirdParty/.../Linux/...")),
	FWorkspaceSyncCategory(FGuid(0x2AF45231, 0x0D75463B, 0xBF9FABB3, 0x231091BB), TEXT("Platform Support: Mac"), TEXT("/Engine/Source/ThirdParty/.../Mac/...")),
	FWorkspaceSyncCategory(FGuid(0xC8CB4934, 0xADE946C9, 0xB6E361A6, 0x59E1FAF5), TEXT("Platform Support: PS4"), TEXT(".../PS4/...")),
	FWorkspaceSyncCategory(FGuid(0xF8AE5AC3, 0xDA2D4719, 0xBABF8A90, 0xD878379E), TEXT("Platform Support: Switch"), TEXT(".../Switch/...")),
	FWorkspaceSyncCategory(FGuid(0x3788A0BC, 0x188C4A0D, 0x950AD681, 0x75F0D110), TEXT("Platform Support: tvOS"), TEXT("/Engine/Source/ThirdParty/.../TVOS/...")),
	FWorkspaceSyncCategory(FGuid(0x1144E719, 0xFCD7491B, 0xB0FC8B4C, 0x3565BF79), TEXT("Platform Support: Win32"), TEXT("/Engine/Source/ThirdParty/.../Win32/...")),
	FWorkspaceSyncCategory(FGuid(0x5206CCEE, 0x90244E36, 0x8B89F5F5, 0xA7D288D2), TEXT("Platform Support: Win64"), TEXT("/Engine/Source/ThirdParty/.../Win64/...")),
	FWorkspaceSyncCategory(FGuid(0x06887423, 0xB0944718, 0x9B55C7A2, 0x1EE67EE4), TEXT("Platform Support: XboxOne"), TEXT(".../XboxOne/...")),
	FWorkspaceSyncCategory(FGuid(0xCFEC942A, 0xBB904F0C, 0xACCF238E, 0xCAAD9430), TEXT("Source Code"), TEXT("/Engine/Source/...")),
};

const TCHAR* FWorkspace::BuildVersionFileName = TEXT("/Engine/Build/Build.version");
const TCHAR* FWorkspace::VersionHeaderFileName = TEXT("/Engine/Source/Runtime/Launch/Resources/Version.h");
const TCHAR* FWorkspace::ObjectVersionFileName = TEXT("/Engine/Source/Runtime/Core/Private/UObject/ObjectVersion.cpp");

FWorkspace* FWorkspace::ActiveWorkspace = nullptr;

//// EWorkspaceUpdateResult ////

FString ToString(EWorkspaceUpdateResult WorkspaceUpdateResult)
{
	switch(WorkspaceUpdateResult)
	{
	case EWorkspaceUpdateResult::Canceled:
		return TEXT("Canceled");
	case EWorkspaceUpdateResult::FailedToSync:
		return TEXT("FailedToSync");
	case EWorkspaceUpdateResult::FilesToResolve:
		return TEXT("FilesToResolve");
	case EWorkspaceUpdateResult::FilesToClobber:
		return TEXT("FilesToClobber");
	case EWorkspaceUpdateResult::FailedToCompile:
		return TEXT("FailedToCompile");
	case EWorkspaceUpdateResult::FailedToCompileWithCleanWorkspace:
		return TEXT("FailedToCompileWithCleanWorkspace");
	case EWorkspaceUpdateResult::Success:
		return TEXT("Success");
	default:
		check(false);
		return FString();
	}
}

bool TryParse(const TCHAR* Text, EWorkspaceUpdateResult& OutWorkspaceUpdateResult)
{
	if(FCString::Stricmp(Text, TEXT("Canceled")) == 0)
	{
		OutWorkspaceUpdateResult = EWorkspaceUpdateResult::Canceled;
		return true;
	}
	if(FCString::Stricmp(Text, TEXT("FailedToSync")) == 0)
	{
		OutWorkspaceUpdateResult = EWorkspaceUpdateResult::FailedToSync;
		return true;
	}
	if(FCString::Stricmp(Text, TEXT("FilesToResolve")) == 0)
	{
		OutWorkspaceUpdateResult = EWorkspaceUpdateResult::FilesToResolve;
		return true;
	}
	if(FCString::Stricmp(Text, TEXT("FilesToClobber")) == 0)
	{
		OutWorkspaceUpdateResult = EWorkspaceUpdateResult::FilesToClobber;
		return true;
	}
	if(FCString::Stricmp(Text, TEXT("FailedToCompile")) == 0)
	{
		OutWorkspaceUpdateResult = EWorkspaceUpdateResult::FailedToCompile;
		return true;
	}
	if(FCString::Stricmp(Text, TEXT("FailedToCompileWithCleanWorkspace")) == 0)
	{
		OutWorkspaceUpdateResult = EWorkspaceUpdateResult::FailedToCompileWithCleanWorkspace;
		return true;
	}
	if(FCString::Stricmp(Text, TEXT("Success")) == 0)
	{
		OutWorkspaceUpdateResult = EWorkspaceUpdateResult::Success;
		return true;
	}
	return false;
}

//// FWorkspaceUpdateContext ////

FWorkspaceUpdateContext::FWorkspaceUpdateContext(int InChangeNumber, EWorkspaceUpdateOptions InOptions, const TArray<FString>& InSyncFilter, const TMap<FGuid, FCustomConfigObject>& InDefaultBuildSteps, const TArray<FCustomConfigObject>& InUserBuildSteps, const TSet<FGuid>& InCustomBuildSteps, const TMap<FString, FString>& InVariables)
	: StartTime(FDateTime::UtcNow())
	, ChangeNumber(InChangeNumber)
	, Options(InOptions)
	, SyncFilter(InSyncFilter)
	, DefaultBuildSteps(InDefaultBuildSteps)
	, UserBuildStepObjects(InUserBuildSteps)
	, CustomBuildSteps(InCustomBuildSteps)
	, Variables(InVariables)
{
}

//// FWorkspaceSyncCategory ////

FWorkspaceSyncCategory::FWorkspaceSyncCategory(const FGuid& InUniqueId)
	: UniqueId(InUniqueId)
	, bEnable(true)
	, Name(TEXT("Unnamed"))
{
}

FWorkspaceSyncCategory::FWorkspaceSyncCategory(const FGuid& InUniqueId, const TCHAR* InName, const TCHAR* InPaths)
	: UniqueId(InUniqueId)
	, bEnable(true)
	, Name(InName)
{
	FString(InPaths).ParseIntoArray(Paths, TEXT(";"));
}

//// FWorkspace ////

FWorkspace::FWorkspace(TSharedRef<FPerforceConnection> InPerforce, const FString& InLocalRootPath, const FString& InSelectedLocalFileName, const FString& InClientRootPath, const FString& InSelectedClientFileName, const FString& InSelectedProjectIdentifier, int InInitialChangeNumber, int InLastBuiltChangeNumber, const FString& InTelemetryProjectPath, TSharedRef<FLineBasedTextWriter> InLog)
	: Perforce(InPerforce)
	, SyncPaths(GetSyncPaths(InClientRootPath, InSelectedClientFileName))
	, LocalRootPath(InLocalRootPath)
	, SelectedLocalFileName(InSelectedLocalFileName)
	, ClientRootPath(InClientRootPath)
	, SelectedClientFileName(InSelectedClientFileName)
	, SelectedProjectIdentifier(InSelectedProjectIdentifier)
	, TelemetryProjectPath(InTelemetryProjectPath)
	, CurrentChangeNumber(InInitialChangeNumber)
	, PendingChangeNumber(InInitialChangeNumber)
	, LastBuiltChangeNumber(InLastBuiltChangeNumber)
	, Log(InLog)
	, bSyncing(false)
	, ProjectConfigFile(ReadProjectConfigFile(InLocalRootPath, InSelectedLocalFileName, InLog.Get()))
	, AbortEvent(FPlatformProcess::GetSynchEventFromPool(true))
	, WorkerThread(nullptr)
{
	ProjectStreamFilter = ReadProjectStreamFilter(InPerforce.Get(), ProjectConfigFile.Get(), AbortEvent, InLog.Get());
	UpdateStatusPanel();
}

FWorkspace::~FWorkspace()
{
	CancelUpdate();
}

void FWorkspace::UpdateStatusPanel()
{
	TSharedPtr<const FCustomConfigSection> ProjectSection = ProjectConfigFile->FindSection(*SelectedProjectIdentifier);

	if (ProjectSection.IsValid())
	{
		PanelColor   = ProjectSection->GetValue(TEXT("StatusPanelColor"), TEXT(""));
		AlertMessage = ProjectSection->GetValue(TEXT("Message"), TEXT(""));
	}
}

FString FWorkspace::GetPanelColor() const
{
	return PanelColor;
}

FString FWorkspace::GetAlertMessage() const
{
	return AlertMessage;
}

TMap<FGuid, FWorkspaceSyncCategory> FWorkspace::GetSyncCategories() const
{
	TMap<FGuid, FWorkspaceSyncCategory> UniqueIdToCategory;

	// Add the default filters
	for(const FWorkspaceSyncCategory& DefaultSyncCategory : DefaultSyncCategories)
	{
		UniqueIdToCategory.Add(DefaultSyncCategory.UniqueId, FWorkspaceSyncCategory(DefaultSyncCategory));
	}

	// Add the custom filters
	TArray<FString> CategoryLines;
	if(ProjectConfigFile->TryGetValues(TEXT("Options.SyncCategory"), CategoryLines))
	{
		for(const FString& CategoryLine : CategoryLines)
		{
			const FCustomConfigObject Object(*CategoryLine);

			FGuid UniqueId;
			if(Object.TryGetValue(TEXT("UniqueId"), UniqueId))
			{
				FWorkspaceSyncCategory* Category = UniqueIdToCategory.Find(UniqueId);
				if(Category == nullptr)
				{
					Category = &(UniqueIdToCategory.Add(UniqueId, FWorkspaceSyncCategory(UniqueId)));
				}

				if(Object.GetValueOrDefault(TEXT("Clear"), false))
				{
					Category->Paths.Empty();
				}

				Category->Name = Object.GetValueOrDefault(TEXT("Name"), *Category->Name);
				Category->bEnable = Object.GetValueOrDefault(TEXT("Enable"), Category->bEnable);

				FString ObjectPaths;
				if(Object.TryGetValue(TEXT("Paths"), ObjectPaths))
				{
					TArray<FString> NewPaths;
					ObjectPaths.ParseIntoArray(NewPaths, TEXT(";"));

					Category->Paths.Append(NewPaths);
					Category->Paths.Sort();

					for(int Idx = Category->Paths.Num() - 1; Idx > 0; Idx--)
					{
						if(Category->Paths[Idx - 1] == Category->Paths[Idx])
						{
							Category->Paths.RemoveAt(Idx);
						}
					}
				}
			}
		}
	}

	return UniqueIdToCategory;
}

TSharedPtr<FCustomConfigFile, ESPMode::ThreadSafe> FWorkspace::GetProjectConfigFile() const
{
	FScopeLock Lock(&CriticalSection);
	return ProjectConfigFile;
}

void FWorkspace::GetProjectStreamFilter(TArray<FString>& Filter)
{
	FScopeLock Lock(&CriticalSection);
	Filter = ProjectStreamFilter;
}

bool FWorkspace::IsBusy() const
{
	return bSyncing;
}

TTuple<FString, float> FWorkspace::GetCurrentProgress() const
{
	FScopeLock Lock(&CriticalSection);
	return Progress.GetCurrent();
}

int FWorkspace::GetCurrentChangeNumber() const
{
	return CurrentChangeNumber;
}

int FWorkspace::GetPendingChangeNumber() const
{
	return PendingChangeNumber;
}

int FWorkspace::GetLastBuiltChangeNumber() const
{
	return LastBuiltChangeNumber;
}

FString FWorkspace::GetClientName() const
{
	return Perforce->ClientName;
}

void FWorkspace::Update(const TSharedRef<FWorkspaceUpdateContext, ESPMode::ThreadSafe>& Context)
{
	// Kill any existing sync
	CancelUpdate();

	// Set the initial progress message
	if(CurrentChangeNumber != Context->ChangeNumber)
	{
		PendingChangeNumber = Context->ChangeNumber;
		if(!EnumHasAnyFlags(Context->Options, EWorkspaceUpdateOptions::SyncSingleChange))
		{
			CurrentChangeNumber = -1;
		}
	}
	Progress.Clear();
	bSyncing = true;

	// Spawn the new thread
	AbortEvent->Reset();
	WorkerThreadContext = Context;
	WorkerThread = FRunnableThread::Create(this, *FString::Printf(TEXT("Update Workspace: %s"), *SelectedClientFileName));
}

void FWorkspace::CancelUpdate()
{
	if(bSyncing)
	{
		Log->WriteLine(TEXT("OPERATION ABORTED"));
		if(WorkerThread != nullptr)
		{
			AbortEvent->Trigger();
			WorkerThread->WaitForCompletion();
			WorkerThread = nullptr;
		}
		PendingChangeNumber = CurrentChangeNumber;
		WorkerThreadContext.Reset();
		bSyncing = false;
		FPlatformAtomics::InterlockedCompareExchangePointer((void**)&ActiveWorkspace, nullptr, this);
	}
}

uint32 FWorkspace::Run()
{
	FString StatusMessage;

	EWorkspaceUpdateResult Result = EWorkspaceUpdateResult::FailedToSync;
	// TODO enable exceptions or dont do this
	//try
	{
		Result = UpdateWorkspaceInternal(*WorkerThreadContext.Get(), StatusMessage);
		if(Result != EWorkspaceUpdateResult::Success)
		{
			Log->Logf(TEXT("%s"), *StatusMessage);
		}
	}
	//catch(FAbortException)
	{
		//StatusMessage = "Canceled.";
		//Log->Logf(TEXT("Canceled."));
	}
//	catch(Exception Ex)
//	{
//		StatusMessage = "Failed with exception - " + Ex.ToString();
//		Log.WriteException(Ex, "Failed with exception");
//	}

	bSyncing = false;
	PendingChangeNumber = CurrentChangeNumber;
	FPlatformAtomics::InterlockedCompareExchangePointer((void**)&ActiveWorkspace, nullptr, this);

	if (OnUpdateComplete)
	{
		OnUpdateComplete(WorkerThreadContext.ToSharedRef(), Result, StatusMessage);
	}

	WorkerThreadContext.Reset();
	return 0;
}

EWorkspaceUpdateResult FWorkspace::UpdateWorkspaceInternal(FWorkspaceUpdateContext& Context, FString& OutStatusMessage)
{
	IFileManager& FileManager = IFileManager::Get();

//	string CmdExe = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.System), "cmd.exe");
#if PLATFORM_WINDOWS
	FString CmdExe = TEXT("C:\\Windows\\System32\\cmd.exe");
#else
	FString CmdExe = TEXT("/usr/bin/env");
#endif
	//if(!FileManager.FileExists(*CmdExe))
	//{
		//OutStatusMessage = FString::Printf(TEXT("Missing %s."), *CmdExe);
		//return EWorkspaceUpdateResult::FailedToSync;
	//}

	TArray<TTuple<FString, FTimespan>> Times;

	int NumFilesSynced = 0;
	if(EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::Sync) || EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::SyncSingleChange))
	{
//		using(TelemetryStopwatch Stopwatch = new TelemetryStopwatch("Sync", TelemetryProjectPath))
		{
			Log->Logf(TEXT("Syncing to %d..."), PendingChangeNumber);

			// Find all the files that are out of date
			Progress.Set(TEXT("Finding files to sync..."));

			// Get the user's sync filter
			TUniquePtr<FFileFilter> UserFilter;
			if(Context.SyncFilter.Num() > 0)
			{
				UserFilter = TUniquePtr<FFileFilter>(new FFileFilter(EFileFilterType::Include));
				for(const FString& UserFilterLine : Context.SyncFilter)
				{
					FString Line = UserFilterLine.TrimStartAndEnd();
					if(Line.Len() > 0 && Line[0] != ';' && Line[0] != '#')
					{
						UserFilter->AddRule(Line);
					}
				}
			}

			// Find all the server changes, and anything that's opened for edit locally. We need to sync files we have open to schedule a resolve.
			TArray<FString> SyncFiles;
			for(const FString& SyncPath : SyncPaths)
			{
				TArray<FPerforceFileRecord> SyncRecords;
				if(!Perforce->SyncPreview(SyncPath, PendingChangeNumber, !EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::Sync), SyncRecords, AbortEvent, Log.Get()))
				{
					OutStatusMessage = FString::Printf(TEXT("Couldn't enumerate changes matching %s."), *SyncPath);
					return EWorkspaceUpdateResult::FailedToSync;
				}

				if(UserFilter.IsValid())
				{
					SyncRecords.RemoveAll([&UserFilter](const FPerforceFileRecord& SyncRecord){ return SyncRecord.ClientPath.Len() > 0 && !UserFilter->Matches(SyncRecord.ClientPath); });
				}

				for(const FPerforceFileRecord& SyncRecord : SyncRecords)
				{
					SyncFiles.Add(SyncRecord.DepotPath);
				}

				TArray<FPerforceFileRecord> OpenRecords;
				if(!Perforce->GetOpenFiles(SyncPath, OpenRecords, AbortEvent, Log.Get()))
				{
					OutStatusMessage = FString::Printf(TEXT("Couldn't find open files matching %s."), *SyncPath);
					return EWorkspaceUpdateResult::FailedToSync;
				}

				// don't force a sync on added files
				for(const FPerforceFileRecord& OpenRecord : OpenRecords)
				{
					if(OpenRecord.Action != TEXT("add") && OpenRecord.Action != TEXT("branch") && OpenRecord.Action != TEXT("move/add"))
					{
						SyncFiles.Add(OpenRecord.DepotPath);
					}
				}
			}

			// Filter out all the binaries that we don't want
			FFileFilter Filter(EFileFilterType::Include);
			Filter.AddRule(FString::Printf(TEXT("...%s"), BuildVersionFileName), EFileFilterType::Exclude);
			Filter.AddRule(FString::Printf(TEXT("...%s"), VersionHeaderFileName), EFileFilterType::Exclude);
			Filter.AddRule(FString::Printf(TEXT("...%s"), ObjectVersionFileName), EFileFilterType::Exclude);
			if(EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::ContentOnly))
			{
				Filter.AddRule(TEXT("*.usf"), EFileFilterType::Exclude);
			}
			SyncFiles.RemoveAll([&Filter](const FString& FileName){ return !Filter.Matches(FileName); });

			// Sync them all
			TArray<FString> TamperedFiles;
			TSet<FString> RemainingFiles(SyncFiles);
			if(!Perforce->Sync(SyncFiles, PendingChangeNumber, [this, &RemainingFiles, &SyncFiles](const FPerforceFileRecord& Record){ UpdateSyncProgress(Record, RemainingFiles, SyncFiles.Num()); }, TamperedFiles, &Context.PerforceSyncOptions, AbortEvent, Log.Get()))
			{
				OutStatusMessage = TEXT("Aborted sync due to errors.");
				return EWorkspaceUpdateResult::FailedToSync;
			}

			// If any files need to be clobbered, defer to the main thread to figure out which ones
			if(TamperedFiles.Num() > 0)
			{
				int NumNewFilesToClobber = 0;
				for(const FString& TamperedFile : TamperedFiles)
				{
					if(!Context.ClobberFiles.Contains(TamperedFile))
					{
						Context.ClobberFiles[TamperedFile] = true;
						NumNewFilesToClobber++;
					}
				}
				if(NumNewFilesToClobber > 0)
				{
					OutStatusMessage = FString::Printf(TEXT("Cancelled sync after checking files to clobber (%d new files)."), NumNewFilesToClobber);
					return EWorkspaceUpdateResult::FilesToClobber;
				}
				for(const FString& TamperedFile : TamperedFiles)
				{
					if(Context.ClobberFiles[TamperedFile] && !Perforce->ForceSync(TamperedFile, PendingChangeNumber, AbortEvent, Log.Get()))
					{
						OutStatusMessage = FString::Printf(TEXT("Couldn't sync %s."), *TamperedFile);
						return EWorkspaceUpdateResult::FailedToSync;
					}
				}
			}

			int VersionChangeNumber = -1;
			if(EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::Sync))
			{
				// Read the new config file
				ProjectConfigFile = ReadProjectConfigFile(LocalRootPath, SelectedLocalFileName, Log.Get());
				ProjectStreamFilter = ReadProjectStreamFilter(Perforce.Get(), ProjectConfigFile.Get(), AbortEvent, Log.Get());

				// Get the branch name
				FString BranchOrStreamName;
				if(!Perforce->GetActiveStream(BranchOrStreamName, AbortEvent, Log.Get()))
				{
					FString DepotFileName;
#if PLATFORM_WINDOWS
					if(!Perforce->ConvertToDepotPath(ClientRootPath + TEXT("/GenerateProjectFiles.bat"), DepotFileName, AbortEvent, Log.Get()))
#else
					if(!Perforce->ConvertToDepotPath(ClientRootPath + TEXT("/GenerateProjectFiles.sh"), DepotFileName, AbortEvent, Log.Get()))
#endif
					{
						OutStatusMessage = FString::Printf(TEXT("Couldn't determine branch name for %s."), *SelectedClientFileName);
						return EWorkspaceUpdateResult::FailedToSync;
					}
					BranchOrStreamName = FPerforceUtils::GetClientOrDepotDirectoryName(*DepotFileName);
				}

				// Get all the paths for code changes
				TArray<FString> CodeFilters;
				for(const FString& SyncPath : SyncPaths)
				{
					const TCHAR* CodeExtensions[] = { TEXT(".cs"), TEXT(".h"), TEXT(".cpp"), TEXT(".usf") };
					for(const TCHAR* CodeExtension : CodeExtensions)
					{
						CodeFilters.Add(FString::Printf(TEXT("%s%s@<=%d"), *SyncPath, CodeExtension, PendingChangeNumber));
					}
				}

				// Find the last code change before this changelist. For consistency in versioning between local builds and precompiled binaries, we need to use the last submitted code changelist as our version number.
				TArray<FPerforceChangeSummary> CodeChanges;
				if(!Perforce->FindChanges(CodeFilters, 1, CodeChanges, AbortEvent, Log.Get()))
				{
					OutStatusMessage = FString::Printf(TEXT("Couldn't determine last code changelist before CL %d."), PendingChangeNumber);
					return EWorkspaceUpdateResult::FailedToSync;
				}
				if(CodeChanges.Num() == 0)
				{
					OutStatusMessage = FString::Printf(TEXT("Could not find any code changes before CL %d."), PendingChangeNumber);
					return EWorkspaceUpdateResult::FailedToSync;
				}

				// Get the last code change
				if(ProjectConfigFile->GetValue(TEXT("Options.VersionToLastCodeChange"), true))
				{
					for(const FPerforceChangeSummary& CodeChange : CodeChanges)
					{
						VersionChangeNumber = FMath::Max(VersionChangeNumber, CodeChange.Number);
					}
				}
				else
				{
					VersionChangeNumber = PendingChangeNumber;
				}

				// Update the version files
				if(ProjectConfigFile->GetValue(TEXT("Options.UseFastModularVersioning"), false))
				{
					TMap<FString, FString> BuildVersionStrings;
					BuildVersionStrings.Add(TEXT("\"Changelist\":"), FString::Printf(TEXT(" %d,"), PendingChangeNumber));
					BuildVersionStrings.Add(TEXT("\"CompatibleChangelist\":"), FString::Printf(TEXT(" %d,"), VersionChangeNumber));
					BuildVersionStrings.Add(TEXT("\"BranchName\":"), FString::Printf(TEXT(" \"%s\""), *BranchOrStreamName.Replace(TEXT("/"), TEXT("+"))));
					BuildVersionStrings.Add(TEXT("\"IsPromotedBuild\":"), TEXT(" 0,"));
					if(!UpdateVersionFile(*(ClientRootPath + BuildVersionFileName), BuildVersionStrings, PendingChangeNumber))
					{
						OutStatusMessage = FString::Printf(TEXT("Failed to update %s."), BuildVersionFileName);
						return EWorkspaceUpdateResult::FailedToSync;
					}

					TMap<FString, FString> VersionHeaderStrings;
					VersionHeaderStrings.Add(TEXT("#define ENGINE_IS_PROMOTED_BUILD"), TEXT(" (0)"));
					VersionHeaderStrings.Add(TEXT("#define BUILT_FROM_CHANGELIST"), TEXT(" 0"));
					VersionHeaderStrings.Add(TEXT("#define BRANCH_NAME"), TEXT(" \"") + BranchOrStreamName.Replace(TEXT("/"), TEXT("+")) + TEXT("\""));
					if(!UpdateVersionFile(*(ClientRootPath + VersionHeaderFileName), VersionHeaderStrings, PendingChangeNumber))
					{
						OutStatusMessage = FString::Printf(TEXT("Failed to update %s."), VersionHeaderFileName);
						return EWorkspaceUpdateResult::FailedToSync;
					}
					if(!UpdateVersionFile(*(ClientRootPath + ObjectVersionFileName), TMap<FString,FString>(), PendingChangeNumber))
					{
						OutStatusMessage = FString::Printf(TEXT("Failed to update %s."), ObjectVersionFileName);
						return EWorkspaceUpdateResult::FailedToSync;
					}
				}
				else
				{
					if(!UpdateVersionFile(*(ClientRootPath + BuildVersionFileName), TMap<FString, FString>(), PendingChangeNumber))
					{
						OutStatusMessage = FString::Printf(TEXT("Failed to update %s"), BuildVersionFileName);
						return EWorkspaceUpdateResult::FailedToSync;
					}

					TMap<FString, FString> VersionStrings;
					VersionStrings.Add(TEXT("#define ENGINE_VERSION"), FString::Printf(TEXT(" %d"), VersionChangeNumber));
					VersionStrings.Add(TEXT("#define ENGINE_IS_PROMOTED_BUILD"), TEXT(" (0)"));
					VersionStrings.Add(TEXT("#define BUILT_FROM_CHANGELIST"), FString::Printf(TEXT(" %d"), VersionChangeNumber));
					VersionStrings.Add(TEXT("#define BRANCH_NAME"), FString::Printf(TEXT(" \"%s\""), *BranchOrStreamName.Replace(TEXT("/"), TEXT("+"))));
					if(!UpdateVersionFile(*(ClientRootPath + VersionHeaderFileName), VersionStrings, PendingChangeNumber))
					{
						OutStatusMessage = FString::Printf(TEXT("Failed to update %s"), VersionHeaderFileName);
						return EWorkspaceUpdateResult::FailedToSync;
					}
					if(!UpdateVersionFile(*(ClientRootPath + ObjectVersionFileName), VersionStrings, PendingChangeNumber))
					{
						OutStatusMessage = FString::Printf(TEXT("Failed to update %s"), ObjectVersionFileName);
						return EWorkspaceUpdateResult::FailedToSync;
					}
				}

				// Remove all the receipts for build targets in this branch
				if(SelectedClientFileName.EndsWith(TEXT(".uproject")))
				{
					Perforce->Sync(FPerforceUtils::GetClientOrDepotDirectoryName(*SelectedClientFileName) + TEXT("/Build/Receipts/...#0"), AbortEvent, Log.Get());
				}
			}

			// Check if there are any files which need resolving
			TArray<FPerforceFileRecord> UnresolvedFiles;
			if(!FindUnresolvedFiles(UnresolvedFiles))
			{
				OutStatusMessage = TEXT("Couldn't get list of unresolved files.");
				return EWorkspaceUpdateResult::FailedToSync;
			}
			if(UnresolvedFiles.Num() > 0 && EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::AutoResolveChanges))
			{
				for(const FPerforceFileRecord& UnresolvedFile : UnresolvedFiles)
				{
					Perforce->AutoResolveFile(UnresolvedFile.DepotPath, AbortEvent, Log.Get());
				}
				if(!FindUnresolvedFiles(UnresolvedFiles))
				{
					OutStatusMessage = "Couldn't get list of unresolved files.";
					return EWorkspaceUpdateResult::FailedToSync;
				}
			}
			if(UnresolvedFiles.Num() > 0)
			{
				Log->Logf(TEXT("%d files need resolving:"), UnresolvedFiles.Num());
				for(const FPerforceFileRecord& UnresolvedFile : UnresolvedFiles)
				{
					Log->Logf(TEXT("  %s"), *UnresolvedFile.ClientPath);
				}
				OutStatusMessage = TEXT("Files need resolving.");
				return EWorkspaceUpdateResult::FilesToResolve;
			}

			// Continue processing sync-only actions
			if (EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::Sync))
			{
				// Execute any project specific post-sync steps
				TArray<FString> PostSyncSteps;
				if(ProjectConfigFile->TryGetValues(TEXT("Sync.Step"), PostSyncSteps))
				{
					Log->Logf(TEXT(""));
					Log->Logf(TEXT("Executing post-sync steps..."));

					TMap<FString, FString> PostSyncVariables(Context.Variables);
					PostSyncVariables.Add("Change", FString::Printf(TEXT("%d"), PendingChangeNumber));
					PostSyncVariables.Add("CodeChange", FString::Printf(TEXT("%d"), VersionChangeNumber));

					for(const FString& PostSyncStep : PostSyncSteps)
					{
						FCustomConfigObject PostSyncStepObject(*PostSyncStep);

						FString ToolFileName = FUtility::ExpandVariables(PostSyncStepObject.GetValueOrDefault(TEXT("FileName"), TEXT("")), &PostSyncVariables);
						if (ToolFileName.Len() > 0)
						{
							FString ToolArguments = FUtility::ExpandVariables(PostSyncStepObject.GetValueOrDefault(TEXT("Arguments"), TEXT("")), &PostSyncVariables);

							Log->Logf(TEXT("post-sync> Running %s %s"), *ToolFileName, *ToolArguments);

							FProgressTextWriter PostSyncWriter(Progress, MakeShared<FPrefixedTextWriter>(TEXT("post-sync> "), Log));

							int ResultFromTool = FUtility::ExecuteProcess(*ToolFileName, *ToolArguments, NULL, AbortEvent, PostSyncWriter);
							if (ResultFromTool != 0)
							{
								OutStatusMessage = FString::Printf(TEXT("Post-sync step terminated with exit code %d."), ResultFromTool);
								return EWorkspaceUpdateResult::FailedToSync;
							}
						}
					}
				}

				// Update the current change number. Everything else happens for the new change.
				CurrentChangeNumber = PendingChangeNumber;
			}

			// Update the timing info
//			Times.Add(TTuple<FString,FTimespan>(TEXT("Sync"), Stopwatch.Stop("Success")));

			// Save the number of files synced
			NumFilesSynced = SyncFiles.Num();
			Log->Logf(TEXT(""));
		}
	}

	// Extract an archive from the depot path
	if(EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::SyncArchives))
	{
		FTelemetryStopwatch Stopwatch(TEXT("Archives"), TelemetryProjectPath);

		// Create the directory for extracted archive manifests
		FString ManifestDirectoryName;
		if(SelectedLocalFileName.EndsWith(TEXT(".uproject")))
		{
			ManifestDirectoryName = FPaths::GetPath(SelectedLocalFileName) / TEXT("Saved/UnrealGameSync");
		}
		else
		{
			ManifestDirectoryName = FPaths::GetPath(SelectedLocalFileName) / TEXT("Engine/Saved/UnrealGameSync");
		}
		IFileManager::Get().MakeDirectory(*ManifestDirectoryName);

		// Sync and extract (or just remove) the given archives
		for(const TTuple<FString, FString>& ArchiveTypeAndDepotPath : Context.ArchiveTypeToDepotPath)
		{
			// Remove any existing binaries
			FString ManifestFileName = ManifestDirectoryName / FString::Printf(TEXT("%s.zipmanifest"), *ArchiveTypeAndDepotPath.Key);
			if(IFileManager::Get().FileExists(*ManifestFileName))
			{
				Log->Logf(TEXT("Removing %s binaries..."), *ArchiveTypeAndDepotPath.Key);
				Progress.Set(*FString::Printf(TEXT("Removing %s binaries..."), *ArchiveTypeAndDepotPath.Key), 0.0f);
//					ArchiveUtils.RemoveExtractedFiles(LocalRootPath, ManifestFileName, Progress, Log);
				Log->Logf(TEXT(""));
			}

			// If we have a new depot path, sync it down and extract it
			if(ArchiveTypeAndDepotPath.Value.Len() > 0)
			{
//					string TempZipFileName = Path.GetTempFileName();
//					try
//					{
//						Log.WriteLine("Syncing {0} binaries...", ArchiveTypeAndDepotPath.Key.ToLowerInvariant());
//						Progress.Set(String.Format("Syncing {0} binaries...", ArchiveTypeAndDepotPath.Key.ToLowerInvariant()), 0.0f);
//						if(!Perforce.PrintToFile(ArchiveTypeAndDepotPath.Value, TempZipFileName, Log) || new FileInfo(TempZipFileName).Length == 0)
//						{
//							StatusMessage = String.Format("Couldn't read {0}", ArchiveTypeAndDepotPath.Value);
//							return WorkspaceUpdateResult.FailedToSync;
//						}
//						ArchiveUtils.ExtractFiles(TempZipFileName, LocalRootPath, ManifestFileName, Progress, Log);
//						Log.WriteLine();
//					}
//					finally
//					{
//						File.SetAttributes(TempZipFileName, FileAttributes.Normal);
//						File.Delete(TempZipFileName);
//					}
			}
		}

		// Add the finish time
		Times.Add(TTuple<FString,FTimespan>(TEXT("Archive"), Stopwatch.Stop(TEXT("Success"))));
	}

	// Take the lock before doing anything else. Building and generating project files can only be done on one workspace at a time.
	if(FPlatformAtomics::InterlockedCompareExchangePointer((void**)&ActiveWorkspace, this, nullptr) != nullptr)
	{
		Log->Logf(TEXT("Waiting for other workspaces to finish..."));
		while(FPlatformAtomics::InterlockedCompareExchangePointer((void**)&ActiveWorkspace, this, nullptr) != nullptr)
		{
			FPlatformProcess::Sleep(0.1f);
		}
	}

	// Generate project files in the workspace
	if(EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::GenerateProjectFiles))
	{
		FTelemetryStopwatch Stopwatch(TEXT("Prj gen"), TelemetryProjectPath);
		Progress.Set(TEXT("Generating project files..."), 0.0f);

		FString ProjectFileArgument;
		if(SelectedLocalFileName.EndsWith(TEXT(".uproject")))
		{
			ProjectFileArgument = FString::Printf(TEXT("\"%s\" "), *SelectedLocalFileName);
		}

#if PLATFORM_WINDOWS
		FString CommandLine = FString::Printf(TEXT("/C \"\"%s\" %s-progress\""), *(LocalRootPath / TEXT("GenerateProjectFiles.bat")), *ProjectFileArgument);
#else
		FString CommandLine = FString::Printf(TEXT("\"%s\" %s-progress"), *(LocalRootPath / TEXT("GenerateProjectFiles.sh")), *ProjectFileArgument);
#endif

		Log->Logf(TEXT("Generating project files..."));
		Log->Logf(TEXT("gpf> Running %s %s"), *CmdExe, *CommandLine);

		FProgressTextWriter ProjectFilesWriter(Progress, MakeShared<FPrefixedTextWriter>(TEXT("gpf> "), Log));

		int GenerateProjectFilesResult = FUtility::ExecuteProcess(*CmdExe, *CommandLine, nullptr, AbortEvent, ProjectFilesWriter);
		if(GenerateProjectFilesResult != 0)
		{
			OutStatusMessage = FString::Printf(TEXT("Failed to generate project files (exit code %d)."), GenerateProjectFilesResult);
			return EWorkspaceUpdateResult::FailedToCompile;
		}

		Log->Logf(TEXT(""));
		Times.Add(TTuple<FString,FTimespan>(TEXT("Prj gen"), Stopwatch.Stop(TEXT("Success"))));
	}

	// Build everything using MegaXGE
	if(EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::Build))
	{
		// Get all the project build steps
		TArray<FCustomConfigObject> ProjectBuildSteps;
		TArray<FString> ProjectBuildStepStrings;
		if(ProjectConfigFile->TryGetValues(TEXT("Build.Step"), ProjectBuildStepStrings))
		{
			for(const FString& ProjectBuildStepString : ProjectBuildStepStrings)
			{
				ProjectBuildSteps.Add(FCustomConfigObject(*ProjectBuildStepString));
			}
		}

		// Compile all the build steps together
		TMap<FGuid, FCustomConfigObject> BuildStepObjects = Context.DefaultBuildSteps;
		FBuildStep::MergeBuildStepObjects(BuildStepObjects, ProjectBuildSteps);
		FBuildStep::MergeBuildStepObjects(BuildStepObjects, Context.UserBuildStepObjects);

		// Construct build steps from them
		TArray<FBuildStep> BuildSteps;
		for(const TTuple<FGuid, FCustomConfigObject>& BuildStepObject : BuildStepObjects)
		{
			BuildSteps.Add(FBuildStep(BuildStepObject.Value));
		}
		BuildSteps.Sort([](const FBuildStep& A, const FBuildStep& B){ return A.OrderIndex < B.OrderIndex; });

		// Remove any that we're not running
		if(Context.CustomBuildSteps.Num() > 0)
		{
			BuildSteps.RemoveAll([&Context](const FBuildStep& Step){ return !Context.CustomBuildSteps.Contains(Step.UniqueId); });
		}
		else if(EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::ScheduledBuild))
		{
			BuildSteps.RemoveAll([](const FBuildStep& Step){ return !Step.bScheduledSync; });
		}
		else
		{
			BuildSteps.RemoveAll([](const FBuildStep& Step){ return !Step.bNormalSync; });
		}

		// Check if the last successful build was before a change that we need to force a clean for
		bool bForceClean = false;
		if(LastBuiltChangeNumber != 0)
		{
			TArray<FString> CleanBuildChanges;
			if(ProjectConfigFile->TryGetValues(TEXT("ForceClean.Changelist"), CleanBuildChanges))
			{
				for(const FString& CleanBuildChange : CleanBuildChanges)
				{
					int ChangeNumber;
					if(FUtility::TryParse(*CleanBuildChange, ChangeNumber))
					{
						if((LastBuiltChangeNumber >= ChangeNumber) != (CurrentChangeNumber >= ChangeNumber))
						{
							Log->Logf(TEXT("Forcing clean build due to changelist %d."), ChangeNumber);
							Log->Logf(TEXT(""));
							bForceClean = true;
							break;
						}
					}
				}
			}
		}

		// Execute them all
		const TCHAR* TelemetryEventName = (Context.UserBuildStepObjects.Num() > 0)? TEXT("CustomBuild") : EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::UseIncrementalBuilds) ? TEXT("Compile") : TEXT("FullCompile");
		FTelemetryStopwatch Stopwatch(TelemetryEventName, TelemetryProjectPath);
		Progress.Set(TEXT("Starting build..."), 0.0f);

		// Check we've built UBT (it should have been compiled by generating project files)
#if PLATFORM_WINDOWS
		FString UnrealBuildToolPath = LocalRootPath / TEXT("Engine/Build/BatchFiles/Build.bat");
#elif PLATFORM_MAC
		FString UnrealBuildToolPath = LocalRootPath / TEXT("Engine/Build/BatchFiles/Mac/Build.sh");
#else
		FString UnrealBuildToolPath = LocalRootPath / TEXT("Engine/Build/BatchFiles/Linux/Build.sh");
#endif
		if(!IFileManager::Get().FileExists(*UnrealBuildToolPath))
		{
			OutStatusMessage = FString::Printf(TEXT("Couldn't find %s"), *UnrealBuildToolPath);
			return EWorkspaceUpdateResult::FailedToCompile;
		}

		// Calculate the total estimated duration
		float TotalEstimatedDuration = 0.0f;
		for(const FBuildStep& Step : BuildSteps)
		{
			TotalEstimatedDuration += Step.EstimatedDuration;
		}

		// Execute all the steps
		float MaxProgressFraction = 0.0f;
		for(const FBuildStep& Step : BuildSteps)
		{
			MaxProgressFraction += (float)Step.EstimatedDuration / (float)FMath::Max(TotalEstimatedDuration, 1.0f);

			Progress.Set(*Step.StatusText);
			Progress.Push(MaxProgressFraction);

			Log->Logf(TEXT("%s"), *Step.StatusText);

			switch(Step.Type)
			{
			case EBuildStepType::Compile:
				{
					FTelemetryStopwatch StepStopwatch(FString::Printf(TEXT("Compile:%s"), *Step.Target), TelemetryProjectPath);

					FString CommandLine = FString::Printf(TEXT("%s %s %s %s -NoHotReloadFromIDE"), *Step.Target, *Step.Platform, *Step.Configuration, *FUtility::ExpandVariables(*Step.Arguments, &Context.Variables));
					if(!EnumHasAnyFlags(Context.Options, EWorkspaceUpdateOptions::UseIncrementalBuilds) || bForceClean)
					{
						Log->Logf(TEXT("ubt> Running %s %s -clean"), *UnrealBuildToolPath, *CommandLine);
						FProgressTextWriter CleanWriter(Progress, MakeShared<FPrefixedTextWriter>(TEXT("ubt> "), Log));
						FUtility::ExecuteProcess(*UnrealBuildToolPath, *(CommandLine + TEXT(" -clean")), nullptr, AbortEvent, CleanWriter);
					}

					Log->Logf(TEXT("ubt> Running %s %s -progress"), *UnrealBuildToolPath, *CommandLine);

					FProgressTextWriter BuildWriter(Progress, MakeShared<FPrefixedTextWriter>(TEXT("ubt> "), Log));
					int ResultFromBuild = FUtility::ExecuteProcess(*UnrealBuildToolPath, *(CommandLine + TEXT(" -progress")), nullptr, AbortEvent, BuildWriter);
					if(ResultFromBuild != 0)
					{
						StepStopwatch.Stop(TEXT("Failed"));
						OutStatusMessage = FString::Printf(TEXT("Failed to compile %s."), *Step.Target);
						return (HasModifiedSourceFiles() || Context.UserBuildStepObjects.Num() > 0)? EWorkspaceUpdateResult::FailedToCompile : EWorkspaceUpdateResult::FailedToCompileWithCleanWorkspace;
					}

					StepStopwatch.Stop(TEXT("Success"));
				}
				break;
			case EBuildStepType::Cook:
				{
					FTelemetryStopwatch StepStopwatch(FString::Printf(TEXT("Cook/Launch: %s"), *FPaths::GetBaseFilename(Step.FileName)), TelemetryProjectPath);

					FString LocalRunUAT = LocalRootPath / TEXT("Engine/Build/BatchFiles/RunUAT.sh");
#if PLATFORM_WINDOWS
					FString Arguments = FString::Printf(TEXT("/C \"\"%s\" -profile=\"%s\"\""), *LocalRunUAT, *(LocalRootPath / Step.FileName));
#else
					FString Arguments = FString::Printf(TEXT("\"%s\" -profile=\"%s\""), *LocalRunUAT, *(LocalRootPath / Step.FileName));
#endif
					Log->Logf(TEXT("uat> Running %s %s"), *LocalRunUAT, *Arguments);

					FProgressTextWriter CookLogWriter(Progress, MakeShared<FPrefixedTextWriter>(TEXT("uat> "), Log));

					int ResultFromUAT = FUtility::ExecuteProcess(*CmdExe, *Arguments, nullptr, AbortEvent, CookLogWriter);
					if(ResultFromUAT != 0)
					{
						StepStopwatch.Stop(TEXT("Failed"));
						OutStatusMessage = FString::Printf(TEXT("Cook failed. (%d)"), ResultFromUAT);
						return EWorkspaceUpdateResult::FailedToCompile;
					}

					StepStopwatch.Stop(TEXT("Success"));
				}
				break;
			case EBuildStepType::Other:
				{
					FTelemetryStopwatch StepStopwatch(FString::Printf(TEXT("Custom: %s"), *FPaths::GetBaseFilename(Step.FileName)), TelemetryProjectPath);

					FString ToolFileName = LocalRootPath / FUtility::ExpandVariables(*Step.FileName, &Context.Variables);
					FString ToolArguments = FUtility::ExpandVariables(*Step.Arguments, &Context.Variables);
					Log->Logf(TEXT("tool> Running %s %s"), *ToolFileName, *ToolArguments);

					if(Step.bUseLogWindow)
					{
						FProgressTextWriter ToolWriter(Progress, MakeShared<FPrefixedTextWriter>(TEXT("tool> "), Log));

						int ResultFromTool = FUtility::ExecuteProcess(*ToolFileName, *ToolArguments, nullptr, AbortEvent, ToolWriter);
						if(ResultFromTool != 0)
						{
							StepStopwatch.Stop(TEXT("Failed"));
							OutStatusMessage = FString::Printf(TEXT("Tool terminated with exit code %d."), ResultFromTool);
							return EWorkspaceUpdateResult::FailedToCompile;
						}
					}
					else
					{
//							using(Process.Start(ToolFileName, ToolArguments))
//							{
//							}
					}

					StepStopwatch.Stop(TEXT("Success"));
				}
				break;
			}

			Log->Logf(TEXT(""));
			Progress.Pop();
		}

		Times.Add(TTuple<FString,FTimespan>(TEXT("Build"), Stopwatch.Stop("Success")));

		// Update the last successful build change number
		if(Context.CustomBuildSteps.Num() == 0)
		{
			LastBuiltChangeNumber = CurrentChangeNumber;
		}
	}

	// Calculate the total time
	long TotalSeconds = 0;
	for(const TTuple<FString, FTimespan>& Time : Times)
	{
		TotalSeconds += (long)Time.Value.GetTotalSeconds();
	}

	// Write out all the timing information
	Log->Logf(TEXT("Total time : %s"), *FormatTime(TotalSeconds));
	for(const TTuple<FString, FTimespan>& Time : Times)
	{
		Log->Logf(TEXT("   %-8s: %s"), *Time.Key, *FormatTime((long)Time.Value.GetTotalSeconds()));
	}
	if(NumFilesSynced > 0)
	{
		Log->Logf(TEXT("{0} files synced."), NumFilesSynced);
	}

	Log->Logf(TEXT(""));
	Log->Logf(TEXT("UPDATE SUCCEEDED"));

	OutStatusMessage = TEXT("Update succeeded");
	return EWorkspaceUpdateResult::Success;
}


TArray<FString> FWorkspace::GetSyncPaths(const FString& ClientRootPath, const FString& SelectedClientFileName)
{
	TArray<FString> SyncPaths;
	if(SelectedClientFileName.EndsWith(TEXT(".uproject")))
	{
		SyncPaths.Add(ClientRootPath + TEXT("/*"));
		SyncPaths.Add(ClientRootPath + TEXT("/Engine/..."));
		SyncPaths.Add(FPerforceUtils::GetClientOrDepotDirectoryName(*SelectedClientFileName) + "/...");
	}
	else
	{
		SyncPaths.Add(ClientRootPath + TEXT("/..."));
	}
	return SyncPaths;
}

TSharedRef<FCustomConfigFile, ESPMode::ThreadSafe> FWorkspace::ReadProjectConfigFile(const FString& LocalRootPath, const FString& SelectedLocalFileName, FLineBasedTextWriter& Log)
{
	// Find the valid config file paths
	TArray<FString> ProjectConfigFileNames;
	ProjectConfigFileNames.Add(LocalRootPath / TEXT("Engine/Programs/UnrealGameSync/UnrealGameSync.ini"));
	ProjectConfigFileNames.Add(LocalRootPath / TEXT("Engine/Programs/UnrealGameSync/NotForLicensees/UnrealGameSync.ini"));
	if(SelectedLocalFileName.EndsWith(TEXT(".uproject")))
	{
		ProjectConfigFileNames.Add(FPaths::GetPath(SelectedLocalFileName) / TEXT("Build/UnrealGameSync.ini"));
		ProjectConfigFileNames.Add(FPaths::GetPath(SelectedLocalFileName) / TEXT("Build/NotForLicensees/UnrealGameSync.ini"));
	}
	else
	{
		ProjectConfigFileNames.Add(LocalRootPath / TEXT("Engine/Programs/UnrealGameSync/DefaultProject.ini"));
		ProjectConfigFileNames.Add(LocalRootPath / TEXT("Engine/Programs/UnrealGameSync/NotForLicensees/DefaultProject.ini"));
	}

	// Read them in
	TSharedRef<FCustomConfigFile, ESPMode::ThreadSafe> ProjectConfig(new FCustomConfigFile());
	for(const FString& ProjectConfigFileName : ProjectConfigFileNames)
	{
		if(IFileManager::Get().FileExists(*ProjectConfigFileName))
		{
			TArray<FString> Lines;
			FFileHelper::LoadFileToStringArray(Lines, *ProjectConfigFileName);
			ProjectConfig->Parse(Lines);
			Log.WriteLine(FString::Printf(TEXT("Read config file from %s"), *ProjectConfigFileName));
		}
	}
	return ProjectConfig;
}

TArray<FString> FWorkspace::ReadProjectStreamFilter(FPerforceConnection& Perforce, const FCustomConfigFile& ProjectConfigFile, FEvent* AbortEvent, FLineBasedTextWriter& Log)
{
	const TCHAR* StreamListDepotPath = ProjectConfigFile.GetValue(TEXT("Options.QuickSelectStreamList"), nullptr);
	if(StreamListDepotPath == nullptr)
	{
		return TArray<FString>();
	}

	TArray<FString> Lines;
	if(!Perforce.Print(StreamListDepotPath, Lines, AbortEvent, Log))
	{
		return TArray<FString>();
	}

	TArray<FString> FilteredLines;
	for(const FString& Line: Lines)
	{
		FString TrimLine = Line.TrimStartAndEnd();
		if(TrimLine.Len() > 0)
		{
			FilteredLines.Add(TrimLine);
		}
	}
	return FilteredLines;
}

FString FWorkspace::FormatTime(long Seconds)
{
	if(Seconds >= 60)
	{
		return FString::Printf(TEXT("%3ldm %02lds"), Seconds / 60, Seconds % 60);
	}
	else
	{
		return FString::Printf(TEXT("     %02lds"), Seconds);
	}
}

bool FWorkspace::HasModifiedSourceFiles() const
{
	TArray<FPerforceFileRecord> OpenFiles;
	if(!Perforce->GetOpenFiles(ClientRootPath + TEXT("/..."), OpenFiles, AbortEvent, Log.Get()))
	{
		return true;
	}
	if(Algo::AnyOf(OpenFiles, [](const FPerforceFileRecord& Record){ return Record.DepotPath.Contains(TEXT("/Source/")); }))
	{
		return true;
	}
	return false;
}

bool FWorkspace::FindUnresolvedFiles(TArray<FPerforceFileRecord>& OutUnresolvedFiles) const
{
	for(const FString& SyncPath : SyncPaths)
	{
		TArray<FPerforceFileRecord> Records;
		if(!Perforce->GetUnresolvedFiles(SyncPath, Records, AbortEvent, Log.Get()))
		{
			Log->Logf(TEXT("Couldn't find open files matching %s"), *SyncPath);
			return false;
		}
		OutUnresolvedFiles.Append(Records);
	}
	return true;
}

void FWorkspace::UpdateSyncProgress(const FPerforceFileRecord& Record, TSet<FString>& RemainingFiles, int NumFiles)
{
	RemainingFiles.Remove(Record.DepotPath);

	FString Message = FString::Printf(TEXT("Syncing files... (%d/%d)"), NumFiles - RemainingFiles.Num(), NumFiles);
	float Fraction = FMath::Min((float)(NumFiles - RemainingFiles.Num()) / (float)NumFiles, 1.0f);
	Progress.Set(*Message, Fraction);

	Log->Logf(TEXT("p4>   %s %s"), *Record.Action, *Record.ClientPath);
}

bool FWorkspace::UpdateVersionFile(const TCHAR* LocalPath, const TMap<FString, FString>& VersionStrings, int ChangeNumber) const
{
	TSharedPtr<FPerforceWhereRecord> WhereRecord;
	if(!Perforce->Where(LocalPath, WhereRecord, AbortEvent, Log.Get()))
	{
		Log->Logf(TEXT("P4 where failed for %s"), *LocalPath);
		return false;
	}

	TArray<FString> Lines;
	if(!Perforce->Print(FString::Printf(TEXT("%s@%d"), *WhereRecord->DepotPath, ChangeNumber), Lines, AbortEvent, Log.Get()))
	{
		Log->Logf(TEXT("Couldn't get default contents of %s"), *WhereRecord->DepotPath);
		return false;
	}

	FString NewFileContents;
	for(const FString& Line : Lines)
	{
		FString NewLine = Line;
		for(const TTuple<FString, FString>& VersionString : VersionStrings)
		{
			if(UpdateVersionLine(NewLine, VersionString.Key, VersionString.Value))
			{
				break;
			}
		}
		NewFileContents += NewLine + LINE_TERMINATOR;
	}

	return WriteVersionFile(*WhereRecord, NewFileContents);
}

bool FWorkspace::WriteVersionFile(const FPerforceWhereRecord& WhereRecord, const FString& NewText) const
{
	IFileManager& FileManager = IFileManager::Get();
	if(FileManager.FileExists(*WhereRecord.LocalPath))
	{
		FString OldText;
		if(FFileHelper::LoadFileToString(OldText, *WhereRecord.LocalPath) && OldText == NewText)
		{
			Log->Logf(TEXT("Ignored %s; contents haven't changed"), *WhereRecord.LocalPath);
			return true;
		}
	}

	FileManager.Delete(*WhereRecord.LocalPath, false, true);

	if(WhereRecord.DepotPath.Len() > 0)
	{
		Perforce->Sync(FString::Printf(TEXT("%s#0"), *WhereRecord.DepotPath), AbortEvent, Log.Get());
	}
	FFileHelper::SaveStringToFile(NewText, *WhereRecord.LocalPath);
	Log->Logf(TEXT("Written %s"), *WhereRecord.LocalPath);
	return true;
}

bool FWorkspace::UpdateVersionLine(FString& Line, const FString& Prefix, const FString& Suffix)
{
	int LineIdx = 0;
	int PrefixIdx = 0;
	for(;;)
	{
		FString PrefixToken;
		if(!ReadToken(Prefix, PrefixIdx, PrefixToken))
		{
			break;
		}

		FString LineToken;
		if(!ReadToken(Line, LineIdx, LineToken) || LineToken != PrefixToken)
		{
			return false;
		}
	}
	Line = Line.Mid(0, LineIdx) + Suffix;
	return true;
}

bool FWorkspace::ReadToken(const FString& Line, int32& LineIdx, FString &OutToken)
{
	for(;; LineIdx++)
	{
		if(LineIdx == Line.Len())
		{
			return false;
		}
		else if(!FChar::IsWhitespace(Line[LineIdx]))
		{
			break;
		}
	}

	int StartIdx = LineIdx++;
	if(FChar::IsAlnum(Line[StartIdx]) || Line[StartIdx] == '_')
	{
		while(LineIdx < Line.Len() && (FChar::IsAlnum(Line[LineIdx]) || Line[LineIdx] == '_'))
		{
			LineIdx++;
		}
	}
	OutToken = Line.Mid(StartIdx, LineIdx - StartIdx);

	return true;
}

} // namespace UGSCore
