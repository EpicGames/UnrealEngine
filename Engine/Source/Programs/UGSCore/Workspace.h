// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CustomConfigFile.h"
#include "Perforce.h"
#include "OutputAdapters.h"
#include "FileFilter.h"
#include "HAL/Runnable.h"
#include "Misc/EnumClassFlags.h"

namespace UGSCore
{

enum class EWorkspaceUpdateOptions
{
	Sync = 0x01,
	SyncSingleChange = 0x02,
	AutoResolveChanges = 0x04,
	GenerateProjectFiles = 0x08,
	SyncArchives = 0x10,
	Build = 0x20,
	UseIncrementalBuilds = 0x40,
	ScheduledBuild = 0x80,
	RunAfterSync = 0x100,
	OpenSolutionAfterSync = 0x200,
	ContentOnly = 0x400
};

ENUM_CLASS_FLAGS(EWorkspaceUpdateOptions)

enum class EWorkspaceUpdateResult
{
	Canceled,
	FailedToSync,
	FilesToResolve,
	FilesToClobber,
	FailedToCompile,
	FailedToCompileWithCleanWorkspace,
	Success,
};

FString ToString(EWorkspaceUpdateResult WorkspaceUpdateResult);
bool TryParse(const TCHAR* Text, EWorkspaceUpdateResult& OutWorkspaceUpdateResult);

struct FWorkspaceUpdateContext
{
	FDateTime StartTime;
	int ChangeNumber;
	EWorkspaceUpdateOptions Options;
	TArray<FString> SyncFilter;
	TMap<FString, FString> ArchiveTypeToDepotPath;
	TMap<FString, bool> ClobberFiles;
	TMap<FGuid,FCustomConfigObject> DefaultBuildSteps;
	TArray<FCustomConfigObject> UserBuildStepObjects;
	TSet<FGuid> CustomBuildSteps;
	TMap<FString, FString> Variables;
	FPerforceSyncOptions PerforceSyncOptions;

	FWorkspaceUpdateContext(int InChangeNumber, EWorkspaceUpdateOptions InOptions, const TArray<FString>& InSyncFilter, const TMap<FGuid, FCustomConfigObject>& InDefaultBuildSteps, const TArray<FCustomConfigObject>& InUserBuildSteps, const TSet<FGuid>& InCustomBuildSteps, const TMap<FString, FString>& InVariables);
};

struct FWorkspaceSyncCategory
{
	FGuid UniqueId;
	bool bEnable;
	FString Name;
	TArray<FString> Paths;

	FWorkspaceSyncCategory(const FGuid& InUniqueId);
	FWorkspaceSyncCategory(const FGuid& InUniqueId, const TCHAR* InName, const TCHAR* InPaths);
};

class FWorkspace : FRunnable
{
public:
	const TSharedRef<FPerforceConnection> Perforce;
	const TArray<FString> SyncPaths;
	const FString LocalRootPath;
	const FString SelectedLocalFileName;
	const FString ClientRootPath;
	const FString SelectedClientFileName;
	const FString SelectedProjectIdentifier;
	const FString TelemetryProjectPath;

	TFunction<void(TSharedRef<FWorkspaceUpdateContext, ESPMode::ThreadSafe>, EWorkspaceUpdateResult, const FString&)> OnUpdateComplete;

	FWorkspace(TSharedRef<FPerforceConnection> InPerforce, const FString& InLocalRootPath, const FString& InSelectedLocalFileName, const FString& InClientRootPath, const FString& InSelectedClientFileName, const FString& InSelectedProjectIdentifier, int InInitialChangeNumber, int InLastBuiltChangeNumber, const FString& InTelemetryProjectPath, TSharedRef<FLineBasedTextWriter> InLog);
	~FWorkspace();

	TMap<FGuid, FWorkspaceSyncCategory> GetSyncCategories() const;
	TSharedPtr<FCustomConfigFile, ESPMode::ThreadSafe> GetProjectConfigFile() const;
	void GetProjectStreamFilter(TArray<FString>& Filter);
	bool IsBusy() const;
	TTuple<FString, float> GetCurrentProgress() const;
	int GetCurrentChangeNumber() const;
	int GetPendingChangeNumber() const;
	int GetLastBuiltChangeNumber() const;
	FString GetClientName() const;

	void Update(const TSharedRef<FWorkspaceUpdateContext, ESPMode::ThreadSafe>& Context);
	void CancelUpdate();

	FString GetPanelColor() const;
	FString GetAlertMessage() const;

private:
	static const TCHAR* DefaultBuildTargets[];
	static const FWorkspaceSyncCategory DefaultSyncCategories[];
	static const TCHAR* BuildVersionFileName;
	static const TCHAR* VersionHeaderFileName;
	static const TCHAR* ObjectVersionFileName;

	int CurrentChangeNumber;
	int PendingChangeNumber;
	int LastBuiltChangeNumber;
	TSharedRef<FLineBasedTextWriter> Log;
	bool bSyncing;
	mutable FCriticalSection CriticalSection;
	TSharedRef<FCustomConfigFile, ESPMode::ThreadSafe> ProjectConfigFile;
	TArray<FString> ProjectStreamFilter;
	TSharedPtr<FWorkspaceUpdateContext, ESPMode::ThreadSafe> WorkerThreadContext;
	FEvent* AbortEvent;
	FRunnableThread* WorkerThread;
	FProgressValue Progress;

	FString PanelColor;
	FString AlertMessage;
	
	static FWorkspace* ActiveWorkspace;

	virtual uint32 Run() override;
	void UpdateWorkspace(FWorkspaceUpdateContext& Context);
	EWorkspaceUpdateResult UpdateWorkspaceInternal(FWorkspaceUpdateContext& Context, FString& OutStatusMessage);

	static TArray<FString> GetSyncPaths(const FString& ClientRootPath, const FString& SelectedClientFileName);
	static TSharedRef<FCustomConfigFile, ESPMode::ThreadSafe> ReadProjectConfigFile(const FString& LocalRootPath, const FString& SelectedLocalFileName, FLineBasedTextWriter& Log);
	static TArray<FString> ReadProjectStreamFilter(FPerforceConnection& Perforce, const FCustomConfigFile& ProjectConfigFile, FEvent* AbortEvent, FLineBasedTextWriter& Log);
	static FString FormatTime(long Seconds);

	void UpdateStatusPanel();
	
	bool HasModifiedSourceFiles() const;
	bool FindUnresolvedFiles(TArray<FPerforceFileRecord>& OutUnresolvedFiles) const;
	void UpdateSyncProgress(const FPerforceFileRecord& Record, TSet<FString>& RemainingFiles, int NumFiles);
	bool UpdateVersionFile(const TCHAR* LocalPath, const TMap<FString, FString>& VersionStrings, int ChangeNumber) const;
	bool WriteVersionFile(const FPerforceWhereRecord& WhereRecord, const FString& NewText) const;
	static bool UpdateVersionLine(FString& Line, const FString& Prefix, const FString& Suffix);
	static bool ReadToken(const FString& Line, int32& LineIdx, FString &OutToken);
};

} // namespace UGSCore
